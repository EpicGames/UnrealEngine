// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptInterface.h"
#include "GameplayTagContainer.h"
#include "GameplayTagAssetInterface.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Templates/SubclassOf.h"
#include "BlueprintGameplayTagLibrary.generated.h"

UCLASS(meta=(ScriptName="GameplayTagLibrary"), MinimalAPI)
class UBlueprintGameplayTagLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	/**
	 * Determine if TagOne matches against TagTwo
	 * 
	 * @param TagOne			Tag to check for match
	 * @param TagTwo			Tag to check match against
	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagOne will include it's parent tags while matching			
	 * 
	 * @return True if TagOne matches TagTwo
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (Keywords = "DoGameplayTagsMatch", BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool MatchesTag(FGameplayTag TagOne, FGameplayTag TagTwo, bool bExactMatch);

	/**
	 * Determine if TagOne matches against any tag in OtherContainer
	 * 
	 * @param TagOne			Tag to check for match
	 * @param OtherContainer	Container to check against.
	 * @param bExactMatch		If true, the tag has to be exactly present, if false then TagOne will include it's parent tags while matching
	 * 
	 * @return True if TagOne matches any tags explicitly present in OtherContainer
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool MatchesAnyTags(FGameplayTag TagOne, const FGameplayTagContainer& OtherContainer, bool bExactMatch);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Equal (GameplayTag)", CompactNodeTitle="==", BlueprintThreadSafe), Category="GameplayTags")
	static GAMEPLAYTAGS_API bool EqualEqual_GameplayTag( FGameplayTag A, FGameplayTag B );
	
	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Not Equal (GameplayTag)", CompactNodeTitle="!=", BlueprintThreadSafe), Category="GameplayTags")
	static GAMEPLAYTAGS_API bool NotEqual_GameplayTag( FGameplayTag A, FGameplayTag B );

	/** Returns true if the passed in gameplay tag is non-null */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool IsGameplayTagValid(FGameplayTag GameplayTag);

	/** Returns FName of this tag */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FName GetTagName(const FGameplayTag& GameplayTag);

	/** Creates a literal FGameplayTag */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTag MakeLiteralGameplayTag(FGameplayTag Value);

	/**
	 * Get the number of gameplay tags in the specified container
	 * 
	 * @param TagContainer	Tag container to get the number of tags from
	 * 
	 * @return The number of tags in the specified container
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API int32 GetNumGameplayTagsInContainer(const FGameplayTagContainer& TagContainer);

	/**
	 * Check if the tag container has the specified tag
	 *
	 * @param TagContainer			Container to check for the tag
	 * @param Tag					Tag to check for in the container
	 * @param bExactMatch			If true, the tag has to be exactly present, if false then TagContainer will include it's parent tags while matching			
	 *
	 * @return True if the container has the specified tag, false if it does not
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (Keywords = "DoesContainerHaveTag", BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool HasTag(const FGameplayTagContainer& TagContainer, FGameplayTag Tag, bool bExactMatch);

	/**
	 * Check if the specified tag container has ANY of the tags in the other container
	 * 
	 * @param TagContainer			Container to check if it matches any of the tags in the other container
	 * @param OtherContainer		Container to check against.
	 * @param bExactMatch			If true, the tag has to be exactly present, if false then TagContainer will include it's parent tags while matching			
	 * 
	 * @return True if the container has ANY of the tags in the other container
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (Keywords = "DoesContainerMatchAnyTagsInContainer", BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool HasAnyTags(const FGameplayTagContainer& TagContainer, const FGameplayTagContainer& OtherContainer, bool bExactMatch);

	/**
	 * Check if the specified tag container has ALL of the tags in the other container
	 * 
	 * @param TagContainer			Container to check if it matches all of the tags in the other container
	 * @param OtherContainer		Container to check against. If this is empty, the check will succeed
	 * @param bExactMatch			If true, the tag has to be exactly present, if false then TagContainer will include it's parent tags while matching			
	 * 
	 * @return True if the container has ALL of the tags in the other container
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (Keywords = "DoesContainerMatchAllTagsInContainer", BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool HasAllTags(const FGameplayTagContainer& TagContainer, const FGameplayTagContainer& OtherContainer, bool bExactMatch);

	/**
	 * Returns a filtered version of this container, returns all tags that match against any of the tags in OtherContainer
	 *
	 * @param OtherContainer		The Container to filter against
	 * @param bExactMatch			If true, the tag has to be exactly present, if false then TagContainer will include it's parent tags while matching
	 *
	 * @return A FGameplayTagContainer containing the filtered tags
	 */
	UFUNCTION(BlueprintPure, Category="GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagContainer Filter(const FGameplayTagContainer& TagContainer, const FGameplayTagContainer& OtherContainer, bool bExactMatch);

	/**
	 * Check if the specified tag query is empty
	 * 
	 * @param TagQuery				Query to check
	 * 
	 * @return True if the query is empty, false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool IsTagQueryEmpty(const FGameplayTagQuery& TagQuery);

	/**
	 * Check if the specified tag container matches the given Tag Query
	 * 
	 * @param TagContainer			Container to check if it matches all of the tags in the other container
	 * @param TagQuery				Query to match against
	 * 
	 * @return True if the container matches the query, false otherwise.
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool DoesContainerMatchTagQuery(const FGameplayTagContainer& TagContainer, const FGameplayTagQuery& TagQuery);

	/**
	 * Get an array of all actors of a specific class (or subclass of that class) which match the specified gameplay tag query.
	 * 
	 * @param ActorClass			Class of actors to fetch
	 * @param GameplayTagQuery		Query to match against
	 * 
	 */
	UFUNCTION(BlueprintCallable, Category="GameplayTags",  meta=(WorldContext="WorldContextObject", DeterminesOutputType="ActorClass", DynamicOutputParam="OutActors"))
	static GAMEPLAYTAGS_API void GetAllActorsOfClassMatchingTagQuery(UObject* WorldContextObject, TSubclassOf<AActor> ActorClass, const FGameplayTagQuery& GameplayTagQuery, TArray<AActor*>& OutActors);

	/**
	 * Adds a single tag to the passed in tag container
	 *
	 * @param InOutTagContainer		The container that will be appended too.
	 * @param Tag					The tag to add to the container
	 */
	UFUNCTION(BlueprintCallable, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API void AddGameplayTag(UPARAM(ref) FGameplayTagContainer& TagContainer, FGameplayTag Tag);

	/**
	 * Remove a single tag from the passed in tag container, returns true if found
	 *
	 * @param InOutTagContainer		The container that will be appended too.
	 * @param Tag					The tag to add to the container
	 */
	UFUNCTION(BlueprintCallable, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API bool RemoveGameplayTag(UPARAM(ref) FGameplayTagContainer& TagContainer, FGameplayTag Tag);

	/**
	 * Appends all tags in the InTagContainer to InOutTagContainer
	 *
	 * @param InOutTagContainer		The container that will be appended too.
	 * @param InTagContainer		The container to append.
	 */
	UFUNCTION(BlueprintCallable, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API void AppendGameplayTagContainers(UPARAM(ref) FGameplayTagContainer& InOutTagContainer, const FGameplayTagContainer& InTagContainer);

	/** Returns true if the values are equal (A == B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Equal (GameplayTagContainer)", CompactNodeTitle="==", BlueprintThreadSafe), Category="GameplayTags")
	static GAMEPLAYTAGS_API bool EqualEqual_GameplayTagContainer( const FGameplayTagContainer& A, const FGameplayTagContainer& B );
	
	/** Returns true if the values are not equal (A != B) */
	UFUNCTION(BlueprintPure, meta=(DisplayName="Not Equal (GameplayTagContainer)", CompactNodeTitle="!=", BlueprintThreadSafe), Category="GameplayTags")
	static GAMEPLAYTAGS_API bool NotEqual_GameplayTagContainer( const FGameplayTagContainer& A, const FGameplayTagContainer& B );

	/** Creates a literal FGameplayTagContainer */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagContainer MakeLiteralGameplayTagContainer(FGameplayTagContainer Value);

	/** Creates a FGameplayTagContainer from the array of passed in tags */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagContainer MakeGameplayTagContainerFromArray(const TArray<FGameplayTag>& GameplayTags);

	/** Creates a FGameplayTagContainer containing a single tag */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagContainer MakeGameplayTagContainerFromTag(FGameplayTag SingleTag);

	/** Breaks tag container into explicit array of tags */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API void BreakGameplayTagContainer(const FGameplayTagContainer& GameplayTagContainer, TArray<FGameplayTag>& GameplayTags);

	/**
	 * Creates a literal FGameplayTagQuery
	 *
	 * @param	TagQuery	value to set the FGameplayTagQuery to
	 *
	 * @return	The literal FGameplayTagQuery
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeGameplayTagQuery(FGameplayTagQuery TagQuery);

	/**
	 * Creates a literal FGameplayTagQuery with a prepopulated AnyTagsMatch expression
	 *
	 * @param	InTags	value to set the FGameplayTagQuery expression
	 *
	 * @return	The literal FGameplayTagQuery
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeGameplayTagQuery_MatchAnyTags(const FGameplayTagContainer& InTags);

	/**
	 * Creates a literal FGameplayTagQuery with a prepopulated AllTagsMatch expression
	 *
	* @param	InTags	value to set the FGameplayTagQuery expression
	 *
	 * @return	The literal FGameplayTagQuery
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeGameplayTagQuery_MatchAllTags(const FGameplayTagContainer& InTags);

	/**
	 * Creates a literal FGameplayTagQuery with a prepopulated NoTagsMatch expression
	 *
	* @param	InTags	value to set the FGameplayTagQuery expression
	 *
	 * @return	The literal FGameplayTagQuery
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe))
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeGameplayTagQuery_MatchNoTags(const FGameplayTagContainer& InTags);
	
	/**
	 * Check Gameplay tags in the interface has all of the specified tags in the tag container (expands to include parents of asset tags)
	 *
	 * @param TagContainerInterface		An Interface to a tag container
	 * @param OtherContainer			A Tag Container
	 *
	 * @return True if the tagcontainer in the interface has all the tags inside the container.
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "TRUE"))
	static GAMEPLAYTAGS_API bool HasAllMatchingGameplayTags(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface, const FGameplayTagContainer& OtherContainer);

	/**
	 * Check Gameplay tags in the interface has ANY of the specified tags in the tag container (expands to include parents of asset tags)
	 *
	 * @param TagContainerInterface		An Interface to a tag container
	 * @param OtherContainer			A Tag Container
	 *
	 * @return True if the tagcontainer in the interface has ANY of the tags inside the container.
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "TRUE"))
	static GAMEPLAYTAGS_API bool HasAnyMatchingGameplayTags(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface, const FGameplayTagContainer& OtherContainer);

	/**
	 * Check if the specified tag container has the specified tag, using the specified tag matching types
	 *
	 * @param TagContainerInterface		An Interface to a tag container
	 * @param Tag						Tag to check for in the container
	 *
	 * @return True if the container has the specified tag, false if it does not
	 */
	UFUNCTION(BlueprintPure, meta = (BlueprintInternalUseOnly = "TRUE"))
	static GAMEPLAYTAGS_API bool DoesTagAssetInterfaceHaveTag(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface, FGameplayTag Tag);

	/** Checks if a gameplay tag's name and a string are not equal to one another */
	UFUNCTION(BlueprintPure, Category = PinOptions, meta = (BlueprintInternalUseOnly = "TRUE"))
	static GAMEPLAYTAGS_API bool NotEqual_TagTag(FGameplayTag A, FString B);

	/** Checks if a gameplay tag containers's name and a string are not equal to one another */
	UFUNCTION(BlueprintPure, Category = PinOptions, meta = (BlueprintInternalUseOnly = "TRUE"))
	static GAMEPLAYTAGS_API bool NotEqual_TagContainerTagContainer(FGameplayTagContainer A, FString B);

	/** @return the tags currently owned by the TagContainerInterface object */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta=(DefaultToSelf=TagContainerInterface))
	static GAMEPLAYTAGS_API UPARAM(DisplayName = "Owned Tags") FGameplayTagContainer GetOwnedGameplayTags(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface);

	/** Converts a UObject to a GameplayTagAssetInterface. This specialty Autocast function exists so we can auto-convert the GameplayTagAssetInterface member functions to static Blueprint functions using redirects. */ 
	UFUNCTION(BlueprintPure, meta=(DisplayName = "To GameplayTagAssetInterface", CompactNodeTitle = "->", BlueprintAutocast), Category="Utilities|Casting")
	static GAMEPLAYTAGS_API TScriptInterface<IGameplayTagAssetInterface> Conv_ObjectToGameplayTagAssetInterface(UObject* InObject);

	/**
	 * Returns an FString listing all of the gameplay tags in the tag container for debugging purposes.
	 *
	 * @param TagContainer	The tag container to get the debug string from.
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe, BlueprintAutocast))
	static GAMEPLAYTAGS_API FString GetDebugStringFromGameplayTagContainer(const FGameplayTagContainer& TagContainer);

	/**
	 * Returns an FString representation of a gameplay tag for debugging purposes.
	 *
	 * @param GameplayTag	The tag to get the debug string from.
	 */
	UFUNCTION(BlueprintPure, Category = "GameplayTags", meta = (BlueprintThreadSafe, BlueprintAutocast))
	static GAMEPLAYTAGS_API FString GetDebugStringFromGameplayTag(FGameplayTag GameplayTag);

};
