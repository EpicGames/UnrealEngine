// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ScriptMacros.h"
#include "UObject/Interface.h"
#include "GameplayTagContainer.h"
#include "GameplayTagAssetInterface.generated.h"

/** Interface for assets which contain gameplay tags */
UINTERFACE(BlueprintType, MinimalAPI, meta=(CannotImplementInterfaceInBlueprint))
class UGameplayTagAssetInterface : public UInterface
{
	GENERATED_UINTERFACE_BODY()
};

class IGameplayTagAssetInterface
{
	GENERATED_IINTERFACE_BODY()

	/**
	 * Get any owned gameplay tags on the asset
	 * 
	 * @param OutTags	[OUT] Set of tags on the asset
	 */
	virtual void GetOwnedGameplayTags(FGameplayTagContainer& TagContainer) const=0;

	/**
	 * Check if the asset has a gameplay tag that matches against the specified tag (expands to include parents of asset tags)
	 * 
	 * @param TagToCheck	Tag to check for a match
	 * 
	 * @return True if the asset has a gameplay tag that matches, false if not
	 */
	UFUNCTION(BlueprintCallable, Category=GameplayTags)
	GAMEPLAYTAGS_API virtual bool HasMatchingGameplayTag(FGameplayTag TagToCheck) const;

	/**
	 * Check if the asset has gameplay tags that matches against all of the specified tags (expands to include parents of asset tags)
	 * 
	 * @param TagContainer			Tag container to check for a match
	 * 
	 * @return True if the asset has matches all of the gameplay tags, will be true if container is empty
	 */
	UFUNCTION(BlueprintCallable, Category=GameplayTags)
	GAMEPLAYTAGS_API virtual bool HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const;

	/**
	 * Check if the asset has gameplay tags that matches against any of the specified tags (expands to include parents of asset tags)
	 * 
	 * @param TagContainer			Tag container to check for a match
	 * 
	 * @return True if the asset has matches any of the gameplay tags, will be false if container is empty
	 */
	UFUNCTION(BlueprintCallable, Category=GameplayTags)
	GAMEPLAYTAGS_API virtual bool HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const;

protected:

	/**
	 * Gets the owned gameplay tags for the asset.  Exposed to allow redirects of existing GetOwnedGameplayTags calls.  In Blueprints, new nodes will use BlueprintGameplayTagLibrary's version.
	 */
	UFUNCTION(BlueprintCallable, Category = GameplayTags, BlueprintInternalUseOnly, meta=(DisplayName="Get Owned Gameplay Tags", AllowPrivateAccess=true))
	GAMEPLAYTAGS_API virtual UPARAM(DisplayName = "Owned Tags") FGameplayTagContainer BP_GetOwnedGameplayTags() const;
};
