// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/Class.h"
#include "Misc/ComparisonUtility.h"
#include "GameplayTagContainer.generated.h"

class UEditableGameplayTagQuery;
struct FGameplayTagContainer;
class FJsonObject;
struct FPropertyTag;

GAMEPLAYTAGS_API DECLARE_LOG_CATEGORY_EXTERN(LogGameplayTags, Log, All);

DECLARE_STATS_GROUP(TEXT("Gameplay Tags"), STATGROUP_GameplayTags, STATCAT_Advanced);

DECLARE_CYCLE_STAT_EXTERN(TEXT("FGameplayTagContainer::DoesTagContainerMatch"), STAT_FGameplayTagContainer_DoesTagContainerMatch, STATGROUP_GameplayTags, GAMEPLAYTAGS_API);

UENUM(BlueprintType)
enum class EGameplayContainerMatchType : uint8
{
	//	Means the filter is populated by any tag matches in this container.
	Any,	

	//	Means the filter is only populated if all of the tags in this container match.
	All		
};

typedef uint16 FGameplayTagNetIndex;
#define INVALID_TAGNETINDEX MAX_uint16

/**
 * A single gameplay tag, which represents a hierarchical name of the form x.y that is registered in the GameplayTagsManager
 * You can filter the gameplay tags displayed in the editor using, meta = (Categories = "Tag1.Tag2.Tag3"))
 */
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/GameplayTags.BlueprintGameplayTagLibrary.MakeLiteralGameplayTag", HasNativeBreak = "/Script/GameplayTags.BlueprintGameplayTagLibrary.GetTagName", DisableSplitPin))
struct FGameplayTag
{
	GENERATED_USTRUCT_BODY()

	/** Constructors */
	FGameplayTag()
	{
	}

	/**
	 * Gets the FGameplayTag that corresponds to the TagName
	 *
	 * @param TagName The Name of the tag to search for
	 * @param ErrorIfNotfound: ensure() that tag exists.
	 * @return Will return the corresponding FGameplayTag or an empty one if not found.
	 */
	static GAMEPLAYTAGS_API FGameplayTag RequestGameplayTag(const FName& TagName, bool ErrorIfNotFound=true);

	/** 
	 * Returns true if this is a valid gameplay tag string (foo.bar.baz). If false, it will fill 
	 * @param TagString String to check for validity
	 * @param OutError If non-null and string invalid, will fill in with an error message
	 * @param OutFixedString If non-null and string invalid, will attempt to fix. Will be empty if no fix is possible
	 * @return True if this can be added to the tag dictionary, false if there's a syntax error
	 */
	static GAMEPLAYTAGS_API bool IsValidGameplayTagString(const FString& TagString, FText* OutError = nullptr, FString* OutFixedString = nullptr);

	/** Operators */
	inline bool operator==(FGameplayTag const& Other) const
	{
		return TagName == Other.TagName;
	}

	inline bool operator!=(FGameplayTag const& Other) const
	{
		return TagName != Other.TagName;
	}

	inline bool operator<(FGameplayTag const& Other) const
	{
		return UE::ComparisonUtility::CompareWithNumericSuffix(TagName, Other.TagName) < 0;
	}

	/**
	 * Determine if this tag matches TagToCheck, expanding out parent tags
	 * "A.1".MatchesTag("A") will return True, "A".MatchesTag("A.1") will return False
	 * If TagToCheck is not Valid it will always return False
	 * 
	 * @return True if this tag matches TagToCheck
	 */
	GAMEPLAYTAGS_API bool MatchesTag(const FGameplayTag& TagToCheck) const;

	/**
	 * Determine if TagToCheck is valid and exactly matches this tag
	 * "A.1".MatchesTagExact("A") will return False
	 * If TagToCheck is not Valid it will always return False
	 * 
	 * @return True if TagToCheck is Valid and is exactly this tag
	 */
	inline bool MatchesTagExact(const FGameplayTag& TagToCheck) const
	{
		if (!TagToCheck.IsValid())
		{
			return false;
		}
		// Only check explicit tag list
		return TagName == TagToCheck.TagName;
	}

	/**
	 * Check to see how closely two FGameplayTags match. Higher values indicate more matching terms in the tags.
	 *
	 * @param TagToCheck	Tag to match against
	 *
	 * @return The depth of the match, higher means they are closer to an exact match
	 */
	GAMEPLAYTAGS_API int32 MatchesTagDepth(const FGameplayTag& TagToCheck) const;

	/**
	 * Checks if this tag matches ANY of the tags in the specified container, also checks against our parent tags
	 * "A.1".MatchesAny({"A","B"}) will return True, "A".MatchesAny({"A.1","B"}) will return False
	 * If ContainerToCheck is empty/invalid it will always return False
	 *
	 * @return True if this tag matches ANY of the tags of in ContainerToCheck
	 */
	GAMEPLAYTAGS_API bool MatchesAny(const FGameplayTagContainer& ContainerToCheck) const;

	/**
	 * Checks if this tag matches ANY of the tags in the specified container, only allowing exact matches
	 * "A.1".MatchesAny({"A","B"}) will return False
	 * If ContainerToCheck is empty/invalid it will always return False
	 *
	 * @return True if this tag matches ANY of the tags of in ContainerToCheck exactly
	 */
	bool MatchesAnyExact(const FGameplayTagContainer& ContainerToCheck) const;

	/** Returns whether the tag is valid or not; Invalid tags are set to NAME_None and do not exist in the game-specific global dictionary */
	inline bool IsValid() const
	{
		return (TagName != NAME_None);
	}

	/** Returns a GameplayTagContainer containing only this tag */
	GAMEPLAYTAGS_API FGameplayTagContainer GetSingleTagContainer() const;

	/** Returns direct parent GameplayTag of this GameplayTag, calling on x.y will return x */
	GAMEPLAYTAGS_API FGameplayTag RequestDirectParent() const;

	/** 
	 * Returns a new tag container that includes this tag and all parent tags as explicitly added tags. 
	 * For example, calling this on x.y.z would return a tag container with x.y.z, x.y, and x
	 */
	GAMEPLAYTAGS_API FGameplayTagContainer GetGameplayTagParents() const;

	/** 
	 * Parses the tag name and fills in UniqueParentTags with raw parent tags, without validating with the tag manager.
	 * For example, calling this on x.y.z would add x.y and x to UniqueParentTags if they were not already in the array
	 */
	GAMEPLAYTAGS_API void ParseParentTags(TArray<FGameplayTag>& UniqueParentTags) const;

	/**
	 * Parses the tag name and returns the name of the leaf.
	 * For example, calling this on x.y.z would return the z component.
	 */
	GAMEPLAYTAGS_API FName GetTagLeafName() const;

	/** Used so we can have a TMap of this struct */
	inline friend uint32 GetTypeHash(const FGameplayTag& Tag)
	{
		return GetTypeHash(Tag.TagName);
	}

	/** Displays gameplay tag as a string for blueprint graph usage */
	inline FString ToString() const
	{
		return TagName.ToString();
	}

	/** Get the tag represented as a name */
	inline FName GetTagName() const
	{
		return TagName;
	}

	friend FArchive& operator<<(FArchive& Ar, FGameplayTag& GameplayTag)
	{
		return Ar << GameplayTag.TagName;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FGameplayTag& GameplayTag)
	{
		Slot << GameplayTag.TagName;
	}

	/** Overridden for fast serialize */
	GAMEPLAYTAGS_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Handles fixup and errors. This is only called when not serializing a full FGameplayTagContainer */
	GAMEPLAYTAGS_API void PostSerialize(const FArchive& Ar);
	GAMEPLAYTAGS_API bool NetSerialize_Packed(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Used to upgrade a Name property to a GameplayTag struct property */
	GAMEPLAYTAGS_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/** Sets from a ImportText string, used in asset registry */
	GAMEPLAYTAGS_API void FromExportString(const FString& ExportString, int32 PortFlags = 0);

	/** Handles importing tag strings without (TagName=) in it */
	GAMEPLAYTAGS_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/** An empty Gameplay Tag */
	static GAMEPLAYTAGS_API const FGameplayTag EmptyTag;

protected:

	bool NetSerialize_ForReplayUsingFastReplication(FArchive& Ar, class UPackageMapClient& PackageMapClient);

	/** Intentionally private so only the tag manager can use */
	GAMEPLAYTAGS_API explicit FGameplayTag(const FName& InTagName);

	/** This Tags Name */
	UPROPERTY(VisibleAnywhere, Category = GameplayTags, SaveGame)
	FName TagName;

	friend class UGameplayTagsManager;
	friend class FGameplayTagRedirectors;
	friend class FNativeGameplayTag;
	friend struct FGameplayTagContainer;
	friend struct FGameplayTagNode;
};

template<>
struct TStructOpsTypeTraits< FGameplayTag > : public TStructOpsTypeTraitsBase2< FGameplayTag >
{
	enum
	{
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithPostSerialize = true,
		WithStructuredSerializeFromMismatchedTag = true,
		WithImportTextItem = true,
	};
};

/** A Tag Container holds a collection of FGameplayTags, tags are included explicitly by adding them, and implicitly from adding child tags */
USTRUCT(BlueprintType, meta = (HasNativeMake = "/Script/GameplayTags.BlueprintGameplayTagLibrary.MakeGameplayTagContainerFromArray", HasNativeBreak = "/Script/GameplayTags.BlueprintGameplayTagLibrary.BreakGameplayTagContainer"))
struct FGameplayTagContainer
{
	GENERATED_USTRUCT_BODY()

	/** Constructors */
	FGameplayTagContainer()
	{
	}

	FGameplayTagContainer(FGameplayTagContainer const& Other)
	{
		*this = Other;
	}

	/** Explicit to prevent people from accidentally using the wrong type of operation */
	explicit FGameplayTagContainer(const FGameplayTag& Tag)
	{
		AddTag(Tag);
	}

	FGameplayTagContainer(FGameplayTagContainer&& Other)
		: GameplayTags(MoveTemp(Other.GameplayTags))
		, ParentTags(MoveTemp(Other.ParentTags))
	{

	}

	~FGameplayTagContainer() = default;

	/** Creates a container from an array of tags, this is more efficient than adding them all individually */
	template<class AllocatorType>
	static FGameplayTagContainer CreateFromArray(const TArray<FGameplayTag, AllocatorType>& SourceTags)
	{
		FGameplayTagContainer Container;
		Container.GameplayTags.Append(SourceTags);
		Container.FillParentTags();
		return Container;
	}

	/** Assignment/Equality operators */
	GAMEPLAYTAGS_API FGameplayTagContainer& operator=(FGameplayTagContainer const& Other);
	GAMEPLAYTAGS_API FGameplayTagContainer& operator=(FGameplayTagContainer&& Other);
	GAMEPLAYTAGS_API bool operator==(FGameplayTagContainer const& Other) const;
	GAMEPLAYTAGS_API bool operator!=(FGameplayTagContainer const& Other) const;

	/**
	 * Determine if TagToCheck is present in this container, also checking against parent tags
	 * {"A.1"}.HasTag("A") will return True, {"A"}.HasTag("A.1") will return False
	 * If TagToCheck is not Valid it will always return False
	 * 
	 * @return True if TagToCheck is in this container, false if it is not
	 */
	inline bool HasTag(const FGameplayTag& TagToCheck) const
	{
		if (!TagToCheck.IsValid())
		{
			return false;
		}
		// Check explicit and parent tag list 
		return GameplayTags.Contains(TagToCheck) || ParentTags.Contains(TagToCheck);
	}

	/**
	 * Determine if TagToCheck is explicitly present in this container, only allowing exact matches
	 * {"A.1"}.HasTagExact("A") will return False
	 * If TagToCheck is not Valid it will always return False
	 * 
	 * @return True if TagToCheck is in this container, false if it is not
	 */
	inline bool HasTagExact(const FGameplayTag& TagToCheck) const
	{
		if (!TagToCheck.IsValid())
		{
			return false;
		}
		// Only check check explicit tag list
		return GameplayTags.Contains(TagToCheck);
	}

	/**
	 * Checks if this container contains ANY of the tags in the specified container, also checks against parent tags
	 * {"A.1"}.HasAny({"A","B"}) will return True, {"A"}.HasAny({"A.1","B"}) will return False
	 * If ContainerToCheck is empty/invalid it will always return False
	 *
	 * @return True if this container has ANY of the tags of in ContainerToCheck
	 */
	inline bool HasAny(const FGameplayTagContainer& ContainerToCheck) const
	{
		if (ContainerToCheck.IsEmpty())
		{
			return false;
		}
		for (const FGameplayTag& OtherTag : ContainerToCheck.GameplayTags)
		{
			if (GameplayTags.Contains(OtherTag) || ParentTags.Contains(OtherTag))
			{
				return true;
			}
		}
		return false;
	}

	/**
	 * Checks if this container contains ANY of the tags in the specified container, only allowing exact matches
	 * {"A.1"}.HasAny({"A","B"}) will return False
	 * If ContainerToCheck is empty/invalid it will always return False
	 *
	 * @return True if this container has ANY of the tags of in ContainerToCheck
	 */
	inline bool HasAnyExact(const FGameplayTagContainer& ContainerToCheck) const
	{
		if (ContainerToCheck.IsEmpty())
		{
			return false;
		}
		for (const FGameplayTag& OtherTag : ContainerToCheck.GameplayTags)
		{
			if (GameplayTags.Contains(OtherTag))
			{
				return true;
			}
		}
		return false;
	}

	/**
	 * Checks if this container contains ALL of the tags in the specified container, also checks against parent tags
	 * {"A.1","B.1"}.HasAll({"A","B"}) will return True, {"A","B"}.HasAll({"A.1","B.1"}) will return False
	 * If ContainerToCheck is empty/invalid it will always return True, because there were no failed checks
	 *
	 * @return True if this container has ALL of the tags of in ContainerToCheck, including if ContainerToCheck is empty
	 */
	inline bool HasAll(const FGameplayTagContainer& ContainerToCheck) const
	{
		if (ContainerToCheck.IsEmpty())
		{
			return true;
		}
		for (const FGameplayTag& OtherTag : ContainerToCheck.GameplayTags)
		{
			if (!GameplayTags.Contains(OtherTag) && !ParentTags.Contains(OtherTag))
			{
				return false;
			}
		}
		return true;
	}

	/**
	 * Checks if this container contains ALL of the tags in the specified container, only allowing exact matches
	 * {"A.1","B.1"}.HasAll({"A","B"}) will return False
	 * If ContainerToCheck is empty/invalid it will always return True, because there were no failed checks
	 *
	 * @return True if this container has ALL of the tags of in ContainerToCheck, including if ContainerToCheck is empty
	 */
	inline bool HasAllExact(const FGameplayTagContainer& ContainerToCheck) const
	{
		if (ContainerToCheck.IsEmpty())
		{
			return true;
		}
		for (const FGameplayTag& OtherTag : ContainerToCheck.GameplayTags)
		{
			if (!GameplayTags.Contains(OtherTag))
			{
				return false;
			}
		}
		return true;
	}

	/** Returns the number of explicitly added tags */
	inline int32 Num() const
	{
		return GameplayTags.Num();
	}

	/** Returns whether the container has any valid tags */
	inline bool IsValid() const
	{
		return GameplayTags.Num() > 0;
	}

	/** Returns true if container is empty */
	inline bool IsEmpty() const
	{
		return GameplayTags.Num() == 0;
	}

	/** Returns a new container explicitly containing the tags of this container and all of their parent tags */
	GAMEPLAYTAGS_API FGameplayTagContainer GetGameplayTagParents() const;

	/**
	 * Returns a filtered version of this container, returns all tags that match against any of the tags in OtherContainer, expanding parents
	 *
	 * @param OtherContainer		The Container to filter against
	 *
	 * @return A FGameplayTagContainer containing the filtered tags
	 */
	GAMEPLAYTAGS_API FGameplayTagContainer Filter(const FGameplayTagContainer& OtherContainer) const;

	/**
	 * Returns a filtered version of this container, returns all tags that match exactly one in OtherContainer
	 *
	 * @param OtherContainer		The Container to filter against
	 *
	 * @return A FGameplayTagContainer containing the filtered tags
	 */
	GAMEPLAYTAGS_API FGameplayTagContainer FilterExact(const FGameplayTagContainer& OtherContainer) const;

	/** 
	 * Checks if this container matches the given query.
	 *
	 * @param Query		Query we are checking against
	 *
	 * @return True if this container matches the query, false otherwise.
	 */
	GAMEPLAYTAGS_API bool MatchesQuery(const struct FGameplayTagQuery& Query) const;

	/** 
	 * Adds all the tags from one container to this container 
	 * NOTE: From set theory, this effectively is the union of the container this is called on with Other.
	 *
	 * @param Other TagContainer that has the tags you want to add to this container 
	 */
	GAMEPLAYTAGS_API void AppendTags(FGameplayTagContainer const& Other);

	/** 
	 * Adds all the tags that match between the two specified containers to this container.  WARNING: This matches any
	 * parent tag in A, not just exact matches!  So while this should be the union of the container this is called on with
	 * the intersection of OtherA and OtherB, it's not exactly that.  Since OtherB matches against its parents, any tag
	 * in OtherA which has a parent match with a parent of OtherB will count.  For example, if OtherA has Color.Green
	 * and OtherB has Color.Red, that will count as a match due to the Color parent match!
	 * If you want an exact match, you need to call A.FilterExact(B) (above) to get the intersection of A with B.
	 * If you need the disjunctive union (the union of two sets minus their intersection), use AppendTags to create
	 * Union, FilterExact to create Intersection, and then call Union.RemoveTags(Intersection).
	 *
	 * @param OtherA TagContainer that has the matching tags you want to add to this container, these tags have their parents expanded
	 * @param OtherB TagContainer used to check for matching tags.  If the tag matches on any parent, it counts as a match.
	 */
	GAMEPLAYTAGS_API void AppendMatchingTags(FGameplayTagContainer const& OtherA, FGameplayTagContainer const& OtherB);

	/**
	 * Add the specified tag to the container
	 *
	 * @param TagToAdd Tag to add to the container
	 */
	GAMEPLAYTAGS_API void AddTag(const FGameplayTag& TagToAdd);

	/**
	 * Add the specified tag to the container without checking for uniqueness
	 *
	 * @param TagToAdd Tag to add to the container
	 * 
	 * Useful when building container from another data struct (TMap for example)
	 */
	GAMEPLAYTAGS_API void AddTagFast(const FGameplayTag& TagToAdd);

	/**
	 * Adds a tag to the container and removes any direct parents, wont add if child already exists
	 *
	 * @param Tag			The tag to try and add to this container
	 * 
	 * @return True if tag was added
	 */
	GAMEPLAYTAGS_API bool AddLeafTag(const FGameplayTag& TagToAdd);

	/**
	 * Tag to remove from the container
	 * 
	 * @param TagToRemove		Tag to remove from the container
	 * @param bDeferParentTags	Skip calling FillParentTags for performance (must be handled by calling code)
	 */
	GAMEPLAYTAGS_API bool RemoveTag(const FGameplayTag& TagToRemove, bool bDeferParentTags=false);

	/**
	 * Removes all tags in TagsToRemove from this container
	 *
	 * @param TagsToRemove	Tags to remove from the container
	 */
	GAMEPLAYTAGS_API void RemoveTags(const FGameplayTagContainer& TagsToRemove);

	/** Remove all tags from the container. Will maintain slack by default */
	GAMEPLAYTAGS_API void Reset(int32 Slack = 0);
	
	/** Serialize the tag container */
	GAMEPLAYTAGS_API bool Serialize(FStructuredArchive::FSlot Slot);

	/** Efficient network serialize, takes advantage of the dictionary */
	GAMEPLAYTAGS_API bool NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess);

	/** Handles fixup after importing from text */
	GAMEPLAYTAGS_API bool ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText);

	/** Fill in the ParentTags array and any other transient parameters */
	GAMEPLAYTAGS_API void PostScriptConstruct();

	/** Returns string version of container in ImportText format */
	GAMEPLAYTAGS_API FString ToString() const;

	/** Sets from a ImportText string, used in asset registry */
	GAMEPLAYTAGS_API void FromExportString(const FString& ExportString, int32 PortFlags = 0);

	/** Returns abbreviated human readable Tag list without parens or property names. If bQuoted is true it will quote each tag */
	GAMEPLAYTAGS_API FString ToStringSimple(bool bQuoted = false) const;

	/** Returns abbreviated human readable Tag list without parens or property names, but will limit each string to specified len.  This is to get around output restrictions*/
	GAMEPLAYTAGS_API TArray<FString> ToStringsMaxLen(int32 MaxLen) const;

	/** Returns human readable description of what match is being looked for on the readable tag list. */
	GAMEPLAYTAGS_API FText ToMatchingText(EGameplayContainerMatchType MatchType, bool bInvertCondition) const;

	/** Gets the explicit list of gameplay tags */
	void GetGameplayTagArray(TArray<FGameplayTag>& InOutGameplayTags) const
	{
		InOutGameplayTags = GameplayTags;
	}

	/** Gets the explicit list of gameplay tags */
	GAMEPLAYTAGS_API const TArray<FGameplayTag>& GetGameplayTagArray() const;

	/** Creates a const iterator for the contents of this array */
	TArray<FGameplayTag>::TConstIterator CreateConstIterator() const
	{
		return GameplayTags.CreateConstIterator();
	}

	bool IsValidIndex(int32 Index) const
	{
		return GameplayTags.IsValidIndex(Index);
	}

	FGameplayTag GetByIndex(int32 Index) const
	{
		if (IsValidIndex(Index))
		{
			return GameplayTags[Index];
		}
		return FGameplayTag();
	}	

	FGameplayTag First() const
	{
		return GameplayTags.Num() > 0 ? GameplayTags[0] : FGameplayTag();
	}

	FGameplayTag Last() const
	{
		return GameplayTags.Num() > 0 ? GameplayTags.Last() : FGameplayTag();
	}

	/** Fills in ParentTags from GameplayTags */
	GAMEPLAYTAGS_API void FillParentTags();

	/** An empty Gameplay Tag Container */
	static GAMEPLAYTAGS_API const FGameplayTagContainer EmptyContainer;

protected:

	/**
	 * If a Tag with the specified tag name explicitly exists, it will remove that tag and return true.  Otherwise, it 
	   returns false.  It does NOT check the TagName for validity (i.e. the tag could be obsolete and so not exist in
	   the table). It also does NOT check parents (because it cannot do so for a tag that isn't in the table).
	   NOTE: This function should ONLY ever be used by GameplayTagsManager when redirecting tags.  Do NOT make this
	   function public!
	 */
	bool RemoveTagByExplicitName(const FName& TagName);

	UE_DEPRECATED(5.4, "Use ParseParentTags or ExtractParentTags instead")
	GAMEPLAYTAGS_API void AddParentsForTag(const FGameplayTag& Tag);

	/** Array of gameplay tags */
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category=GameplayTags, SaveGame)
	TArray<FGameplayTag> GameplayTags;

	/** Array of expanded parent tags, in addition to GameplayTags. Used to accelerate parent searches. May contain duplicates in some cases */
	UPROPERTY(Transient)
	TArray<FGameplayTag> ParentTags;

	friend class UGameplayTagsManager;
	friend class FGameplayTagRedirectors;
	friend struct FGameplayTagQuery;
	friend struct FGameplayTagQueryExpression;
	friend struct FGameplayTagNode;
	friend struct FGameplayTag;
	
private:

	/**
	 * DO NOT USE DIRECTLY
	 * STL-like iterators to enable range-based for loop support.
	 */
	
	inline friend TArray<FGameplayTag>::TConstIterator begin(const FGameplayTagContainer& Array) { return Array.CreateConstIterator(); }
	inline friend TArray<FGameplayTag>::TConstIterator end(const FGameplayTagContainer& Array) { return TArray<FGameplayTag>::TConstIterator(Array.GameplayTags, Array.GameplayTags.Num()); }
};

inline bool FGameplayTag::MatchesAnyExact(const FGameplayTagContainer& ContainerToCheck) const
{
	if (ContainerToCheck.IsEmpty())
	{
		return false;
	}
	return ContainerToCheck.GameplayTags.Contains(*this);
}

template<>
struct TStructOpsTypeTraits<FGameplayTagContainer> : public TStructOpsTypeTraitsBase2<FGameplayTagContainer>
{
	enum
	{
		WithStructuredSerializer = true,
		WithIdenticalViaEquality = true,
		WithNetSerializer = true,
		WithNetSharedSerialization = true,
		WithImportTextItem = true,
		WithCopy = true,
		WithPostScriptConstruct = true,
	};
	static constexpr EPropertyObjectReferenceType WithSerializerObjectReferences = EPropertyObjectReferenceType::None;
};

/** Class that can be subclassed by a game/plugin to allow easily adding native gameplay tags at startup */
struct FGameplayTagNativeAdder
{
	GAMEPLAYTAGS_API FGameplayTagNativeAdder();

	virtual GAMEPLAYTAGS_API ~FGameplayTagNativeAdder();

	virtual void AddTags() = 0;
	
	FDelegateHandle AddTagsHandle;
};

/** Helper struct: drop this in another struct to get an embedded create new tag widget. */
USTRUCT()
struct FGameplayTagCreationWidgetHelper
{
	GENERATED_USTRUCT_BODY()
};

/** Enumerates the list of supported query expression types. */
UENUM()
enum class EGameplayTagQueryExprType : uint8
{
	Undefined = 0,
	AnyTagsMatch,
	AllTagsMatch,
	NoTagsMatch,
	AnyExprMatch,
	AllExprMatch,
	NoExprMatch,
	AnyTagsExactMatch,
	AllTagsExactMatch
};

namespace EGameplayTagQueryStreamVersion
{
	enum Type
	{
		InitialVersion = 0,

		// -----<new versions can be added before this line>-------------------------------------------------
		// - this needs to be the last line (see note below)
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};
}

/**
 * An FGameplayTagQuery is a logical query that can be run against an FGameplayTagContainer.  A query that succeeds is said to "match".
 * Queries are logical expressions that can test the intersection properties of another tag container (all, any, or none), or the matching state of a set of sub-expressions
 * (all, any, or none). This allows queries to be arbitrarily recursive and very expressive.  For instance, if you wanted to test if a given tag container contained tags 
 * ((A && B) || (C)) && (!D), you would construct your query in the form ALL( ANY( ALL(A,B), ALL(C) ), NONE(D) )
 * 
 * You can expose the query structs to Blueprints and edit them with a custom editor, or you can construct them natively in code. 
 * 
 * Example of how to build a query via code:
 *	FGameplayTagQuery Q;
 *	Q.BuildQuery(
 *		FGameplayTagQueryExpression()
 * 		.AllTagsMatch()
 *		.AddTag(FGameplayTag::RequestGameplayTag(FName("Animal.Mammal.Dog.Corgi")))
 *		.AddTag(FGameplayTag::RequestGameplayTag(FName("Plant.Tree.Spruce")))
 *		);
 * 
 * Queries are internally represented as a byte stream that is memory-efficient and can be evaluated quickly at runtime.
 * Note: these have an extensive details and graph pin customization for editing, so there is no need to expose the internals to Blueprints.
 */
USTRUCT(BlueprintType, meta=(HasNativeMake="/Script/GameplayTags.BlueprintGameplayTagLibrary.MakeGameplayTagQuery"))
struct FGameplayTagQuery
{
	GENERATED_BODY();

public:
	GAMEPLAYTAGS_API FGameplayTagQuery();

	GAMEPLAYTAGS_API FGameplayTagQuery(FGameplayTagQuery const& Other);
	GAMEPLAYTAGS_API FGameplayTagQuery(FGameplayTagQuery&& Other);

	/** Assignment/Equality operators */
	GAMEPLAYTAGS_API FGameplayTagQuery& operator=(FGameplayTagQuery const& Other);
	GAMEPLAYTAGS_API FGameplayTagQuery& operator=(FGameplayTagQuery&& Other);

	GAMEPLAYTAGS_API bool operator==(const FGameplayTagQuery& Other) const;
	GAMEPLAYTAGS_API bool operator!=(const FGameplayTagQuery& Other) const;

private:
	// Note: Properties need to be editable to allow FComponentPropertyWriter to serialize them, but are hidden in the editor by the customizations mentioned above.

	/** Versioning for future token stream protocol changes. See EGameplayTagQueryStreamVersion. */
	UPROPERTY(EditAnywhere, Category = Hidden)
	int32 TokenStreamVersion;

	/** List of tags referenced by this entire query. Token stream stored indices into this list. */
	UPROPERTY(EditAnywhere, Category = Hidden)
	TArray<FGameplayTag> TagDictionary;

	/** Stream representation of the actual hierarchical query */
	UPROPERTY(EditAnywhere, Category = Hidden)
	TArray<uint8> QueryTokenStream;

	/** User-provided string describing the query */
	UPROPERTY(EditAnywhere, Category = Hidden)
	FString UserDescription;

	/** Auto-generated string describing the query */
	UPROPERTY(EditAnywhere, Category = Hidden)
	FString AutoDescription;

	/** Returns a gameplay tag from the tag dictionary */
	FGameplayTag GetTagFromIndex(int32 TagIdx) const
	{
		ensure(TagDictionary.IsValidIndex(TagIdx));
		return TagDictionary[TagIdx];
	}

public:

	/** Replaces existing tags with passed in tags. Does not modify the tag query expression logic. Useful when you need to cache off and update often used query. Must use same sized tag container! */
	void ReplaceTagsFast(FGameplayTagContainer const& Tags)
	{
		ensure(Tags.Num() == TagDictionary.Num());
		TagDictionary.Reset();
		TagDictionary.Append(Tags.GameplayTags);
	}

	/** Replaces existing tags with passed in tag. Does not modify the tag query expression logic. Useful when you need to cache off and update often used query. */		 
	void ReplaceTagFast(FGameplayTag const& Tag)
	{
		ensure(1 == TagDictionary.Num());
		TagDictionary.Reset();
		TagDictionary.Add(Tag);
	}

	/** Returns true if the given tags match this query, or false otherwise. */
	GAMEPLAYTAGS_API bool Matches(FGameplayTagContainer const& Tags) const;

	/** Returns true if this query is empty, false otherwise. */
	GAMEPLAYTAGS_API bool IsEmpty() const;

	/** Resets this query to its default empty state. */
	GAMEPLAYTAGS_API void Clear();

	/** Creates this query with the given root expression. */
	GAMEPLAYTAGS_API void Build(struct FGameplayTagQueryExpression& RootQueryExpr, FString InUserDescription = FString());

	/** Static function to assemble and return a query. */
	static GAMEPLAYTAGS_API FGameplayTagQuery BuildQuery(struct FGameplayTagQueryExpression& RootQueryExpr, FString InDescription = FString());

	/** Builds a FGameplayTagQueryExpression from this query. */
	GAMEPLAYTAGS_API void GetQueryExpr(struct FGameplayTagQueryExpression& OutExpr) const;

	/** Serialize the tag query */
	GAMEPLAYTAGS_API void Serialize(FArchive& Ar);
	
	/** Sets the user-specified description of this query. */
	void SetUserDescription(const FString& InUserDescription) { UserDescription = InUserDescription; }

	/** Returns description string. */
	const FString& GetDescription() const { return UserDescription.IsEmpty() ? AutoDescription : UserDescription; };

	/** Gets the explicit list of all unique gameplay tags referenced by the query. */
	void GetGameplayTagArray(TArray<FGameplayTag>& OutGameplayTags) const
	{
		OutGameplayTags = TagDictionary;
	}

	/** Gets the explicit list of all unique gameplay tags referenced by the query. */
	GAMEPLAYTAGS_API const TArray<FGameplayTag>& GetGameplayTagArray() const;

#if WITH_EDITOR
	/** Creates this query based on the given EditableQuery object */
	GAMEPLAYTAGS_API void BuildFromEditableQuery(class UEditableGameplayTagQuery& EditableQuery); 

	/** Creates editable query object tree based on this query */
	GAMEPLAYTAGS_API UEditableGameplayTagQuery* CreateEditableQuery() const;
#endif // WITH_EDITOR

	static GAMEPLAYTAGS_API const FGameplayTagQuery EmptyQuery;

	/**
	* Shortcuts for easily creating common query types
	* @todo: add more as dictated by use cases
	*/

	/** Creates a tag query that will match if there are any common tags between the given tags and the tags being queries against. */
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeQuery_MatchAnyTags(FGameplayTagContainer const& InTags);
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeQuery_MatchAllTags(FGameplayTagContainer const& InTags);
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeQuery_MatchNoTags(FGameplayTagContainer const& InTags);
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeQuery_ExactMatchAnyTags(FGameplayTagContainer const& InTags);
	static GAMEPLAYTAGS_API FGameplayTagQuery MakeQuery_ExactMatchAllTags(FGameplayTagContainer const& InTags);

	static GAMEPLAYTAGS_API FGameplayTagQuery MakeQuery_MatchTag(FGameplayTag const& InTag);

	friend class FQueryEvaluator;
};

struct FGameplayTagQueryExpression
{
	/** 
	 * Fluid syntax approach for setting the type of this expression. 
	 */

	FGameplayTagQueryExpression& AnyTagsMatch()
	{
		ExprType = EGameplayTagQueryExprType::AnyTagsMatch;
		return *this;
	}

	FGameplayTagQueryExpression& AllTagsMatch()
	{
		ExprType = EGameplayTagQueryExprType::AllTagsMatch;
		return *this;
	}

	FGameplayTagQueryExpression& NoTagsMatch()
	{
		ExprType = EGameplayTagQueryExprType::NoTagsMatch;
		return *this;
	}

	FGameplayTagQueryExpression& AnyTagsExactMatch()
	{
		ExprType = EGameplayTagQueryExprType::AnyTagsExactMatch;
		return *this;
	}

	FGameplayTagQueryExpression& AllTagsExactMatch()
	{
		ExprType = EGameplayTagQueryExprType::AllTagsExactMatch;
		return *this;
	}

	FGameplayTagQueryExpression& AnyExprMatch()
	{
		ExprType = EGameplayTagQueryExprType::AnyExprMatch;
		return *this;
	}

	FGameplayTagQueryExpression& AllExprMatch()
	{
		ExprType = EGameplayTagQueryExprType::AllExprMatch;
		return *this;
	}

	FGameplayTagQueryExpression& NoExprMatch()
	{
		ExprType = EGameplayTagQueryExprType::NoExprMatch;
		return *this;
	}

	FGameplayTagQueryExpression& AddTag(TCHAR const* TagString)
	{
		return AddTag(FName(TagString));
	}
	GAMEPLAYTAGS_API FGameplayTagQueryExpression& AddTag(FName TagName);
	FGameplayTagQueryExpression& AddTag(FGameplayTag Tag)
	{
		ensure(UsesTagSet());
		TagSet.Add(Tag);
		return *this;
	}
	
	FGameplayTagQueryExpression& AddTags(FGameplayTagContainer const& Tags)
	{
		ensure(UsesTagSet());
		TagSet.Append(Tags.GameplayTags);
		return *this;
	}

	FGameplayTagQueryExpression& AddExpr(FGameplayTagQueryExpression& Expr)
	{
		ensure(UsesExprSet());
		ExprSet.Add(Expr);
		return *this;
	}
	
	/** Writes this expression to the given token stream. */
	GAMEPLAYTAGS_API void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary) const;

	/** Which type of expression this is. */
	EGameplayTagQueryExprType ExprType;

	/** Expression list, for expression types that need it */
	TArray<struct FGameplayTagQueryExpression> ExprSet;

	/** Tag list, for expression types that need it */
	TArray<FGameplayTag> TagSet;

	/** Returns true if this expression uses the tag data. */
	inline bool UsesTagSet() const
	{
		return (ExprType == EGameplayTagQueryExprType::AllTagsMatch) || (ExprType == EGameplayTagQueryExprType::AnyTagsMatch) || (ExprType == EGameplayTagQueryExprType::NoTagsMatch) || (ExprType == EGameplayTagQueryExprType::AnyTagsExactMatch) || (ExprType == EGameplayTagQueryExprType::AllTagsExactMatch);
	}
	/** Returns true if this expression uses the expression list data. */
	inline bool UsesExprSet() const
	{
		return (ExprType == EGameplayTagQueryExprType::AllExprMatch) || (ExprType == EGameplayTagQueryExprType::AnyExprMatch) || (ExprType == EGameplayTagQueryExprType::NoExprMatch);
	}

	/** Converts the existing TagQueryExpression into a json object. Returns true on success */
	GAMEPLAYTAGS_API bool ConvertToJsonObject(TSharedRef<FJsonObject>& OutObject) const;

	/** Uses the input json object and fills out the OutQueryExpression with the data. */
	static GAMEPLAYTAGS_API bool MakeFromJsonObject(const TSharedRef<FJsonObject>& InObject, FGameplayTagQueryExpression& OutQueryExpression);
};

template<>
struct TStructOpsTypeTraits<FGameplayTagQuery> : public TStructOpsTypeTraitsBase2<FGameplayTagQuery>
{
	enum
	{
		WithCopy = true
	};
};



/** 
 * This is an editor-only representation of a query, designed to be editable with a typical property window. 
 * To edit a query in the editor, an FGameplayTagQuery is converted to a set of UObjects and edited,  When finished,
 * the query struct is rewritten and these UObjects are discarded.
 * This query representation is not intended for runtime use.
 */
UCLASS(editinlinenew, collapseCategories, Transient, MinimalAPI) 
class UEditableGameplayTagQuery : public UObject
{
	GENERATED_BODY()

public:
	/** User-supplied description, shown in property details. Auto-generated description is shown if not supplied. */
	UPROPERTY(EditDefaultsOnly, Category = Query)
	FString UserDescription;

	/** Automatically-generated description */
	FString AutoDescription;

	/** The base expression of this query. */
	UPROPERTY(EditDefaultsOnly, Instanced, Category = Query)
	TObjectPtr<class UEditableGameplayTagQueryExpression> RootExpression;

#if WITH_EDITOR
	/** Converts this editor query construct into the runtime-usable token stream. */
	GAMEPLAYTAGS_API void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString=nullptr) const;

	/** Generates and returns the export text for this query. */
	GAMEPLAYTAGS_API FString GetTagQueryExportText(FGameplayTagQuery const& TagQuery);
#endif  // WITH_EDITOR

private:
	/** Property to hold a gameplay tag query so we can use the exporttext path to get a string representation. */
	UPROPERTY()
	FGameplayTagQuery TagQueryExportText_Helper;
};

UCLASS(abstract, editinlinenew, collapseCategories, Transient, MinimalAPI)
class UEditableGameplayTagQueryExpression : public UObject
{
	GENERATED_BODY()

#if WITH_EDITOR
public:
	/** Converts this editor query construct into the runtime-usable token stream. */
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString=nullptr) const {};

protected:
	GAMEPLAYTAGS_API void EmitTagTokens(FGameplayTagContainer const& TagsToEmit, TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const;
	GAMEPLAYTAGS_API void EmitExprListTokens(TArray<UEditableGameplayTagQueryExpression*> const& ExprList, TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta=(DisplayName="Any Tags Match"))
class UEditableGameplayTagQueryExpression_AnyTagsMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditDefaultsOnly, Category = Expr)
	FGameplayTagContainer Tags;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "All Tags Match"))
class UEditableGameplayTagQueryExpression_AllTagsMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditDefaultsOnly, Category = Expr)
	FGameplayTagContainer Tags;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "No Tags Match"))
class UEditableGameplayTagQueryExpression_NoTagsMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditDefaultsOnly, Category = Expr)
	FGameplayTagContainer Tags;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "Any Tags Exact Match"))
class UEditableGameplayTagQueryExpression_AnyTagsExactMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditDefaultsOnly, Category = Expr)
	FGameplayTagContainer Tags;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "All Tags Exact Match"))
class UEditableGameplayTagQueryExpression_AllTagsExactMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditDefaultsOnly, Category = Expr)
	FGameplayTagContainer Tags;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "Any Expressions Match"))
class UEditableGameplayTagQueryExpression_AnyExprMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Instanced, Category = Expr)
	TArray<TObjectPtr<UEditableGameplayTagQueryExpression>> Expressions;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "All Expressions Match"))
class UEditableGameplayTagQueryExpression_AllExprMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Instanced, Category = Expr)
	TArray<TObjectPtr<UEditableGameplayTagQueryExpression>> Expressions;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

UCLASS(BlueprintType, editinlinenew, collapseCategories, meta = (DisplayName = "No Expressions Match"))
class UEditableGameplayTagQueryExpression_NoExprMatch : public UEditableGameplayTagQueryExpression
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Instanced, Category = Expr)
	TArray<TObjectPtr<UEditableGameplayTagQueryExpression>> Expressions;

#if WITH_EDITOR
	virtual void EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString = nullptr) const override;
#endif  // WITH_EDITOR
};

