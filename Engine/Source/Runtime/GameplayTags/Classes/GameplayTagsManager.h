// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/LruCache.h"
#include "CoreMinimal.h"
#include "Misc/EnumRange.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "GameplayTagContainer.h"
#include "Engine/DataTable.h"
#include "Templates/UniquePtr.h"
#include "Misc/ScopeLock.h"
#include "Misc/TransactionallySafeCriticalSection.h"
#if WITH_EDITOR
#include "Hash/Blake3.h"
#endif

#include "GameplayTagsManager.generated.h"

class FCbFieldViewIterator;
class FNativeGameplayTag;
class UGameplayTagsList;
struct FStreamableHandle;

#if WITH_EDITOR
namespace GameplayTagUtil { struct FGameplayTagCookDependency; }
namespace UE::Cook { class FCookDependency; }
namespace UE::Cook { class ICookInfo; }
namespace UE::Cook { struct FCookDependencyContext; }
#endif

/** Simple struct for a table row in the gameplay tag table and element in the ini list */
USTRUCT()
struct FGameplayTagTableRow : public FTableRowBase
{
	GENERATED_USTRUCT_BODY()

	/** Tag specified in the table */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=GameplayTag)
	FName Tag;

	/** Developer comment clarifying the usage of a particular tag, not user facing */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=GameplayTag)
	FString DevComment;

	/** Constructors */
	FGameplayTagTableRow() {}
	FGameplayTagTableRow(FName InTag, const FString& InDevComment = TEXT("")) : Tag(InTag), DevComment(InDevComment) {}
	GAMEPLAYTAGS_API FGameplayTagTableRow(FGameplayTagTableRow const& Other);

	/** Assignment/Equality operators */
	GAMEPLAYTAGS_API FGameplayTagTableRow& operator=(FGameplayTagTableRow const& Other);
	GAMEPLAYTAGS_API bool operator==(FGameplayTagTableRow const& Other) const;
	GAMEPLAYTAGS_API bool operator!=(FGameplayTagTableRow const& Other) const;
	GAMEPLAYTAGS_API bool operator<(FGameplayTagTableRow const& Other) const;
};

/** Simple struct for a table row in the restricted gameplay tag table and element in the ini list */
USTRUCT()
struct FRestrictedGameplayTagTableRow : public FGameplayTagTableRow
{
	GENERATED_USTRUCT_BODY()

	/** Tag specified in the table */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = GameplayTag)
	bool bAllowNonRestrictedChildren;

	/** Constructors */
	FRestrictedGameplayTagTableRow() : bAllowNonRestrictedChildren(false) {}
	FRestrictedGameplayTagTableRow(FName InTag, const FString& InDevComment = TEXT(""), bool InAllowNonRestrictedChildren = false) : FGameplayTagTableRow(InTag, InDevComment), bAllowNonRestrictedChildren(InAllowNonRestrictedChildren) {}
	GAMEPLAYTAGS_API FRestrictedGameplayTagTableRow(FRestrictedGameplayTagTableRow const& Other);

	/** Assignment/Equality operators */
	GAMEPLAYTAGS_API FRestrictedGameplayTagTableRow& operator=(FRestrictedGameplayTagTableRow const& Other);
	GAMEPLAYTAGS_API bool operator==(FRestrictedGameplayTagTableRow const& Other) const;
	GAMEPLAYTAGS_API bool operator!=(FRestrictedGameplayTagTableRow const& Other) const;
};

UENUM()
enum class EGameplayTagSourceType : uint8
{
	Native,				// Was added from C++ code
	DefaultTagList,		// The default tag list in DefaultGameplayTags.ini
	TagList,			// Another tag list from an ini in tags/*.ini
	RestrictedTagList,	// Restricted tags from an ini
	DataTable,			// From a DataTable
	Invalid,			// Not a real source
};

ENUM_RANGE_BY_COUNT(EGameplayTagSourceType, EGameplayTagSourceType::Invalid);

UENUM()
enum class EGameplayTagSelectionType : uint8
{
	None,
	NonRestrictedOnly,
	RestrictedOnly,
	All
};

/** Struct defining where gameplay tags are loaded/saved from. Mostly for the editor */
USTRUCT()
struct FGameplayTagSource
{
	GENERATED_USTRUCT_BODY()

	/** Name of this source */
	UPROPERTY()
	FName SourceName;

	/** Type of this source */
	UPROPERTY()
	EGameplayTagSourceType SourceType;

	/** If this is bound to an ini object for saving, this is the one */
	UPROPERTY()
	TObjectPtr<class UGameplayTagsList> SourceTagList;

	/** If this has restricted tags and is bound to an ini object for saving, this is the one */
	UPROPERTY()
	TObjectPtr<class URestrictedGameplayTagsList> SourceRestrictedTagList;

	FGameplayTagSource() 
		: SourceName(NAME_None), SourceType(EGameplayTagSourceType::Invalid), SourceTagList(nullptr), SourceRestrictedTagList(nullptr)
	{
	}

	FGameplayTagSource(FName InSourceName, EGameplayTagSourceType InSourceType, UGameplayTagsList* InSourceTagList = nullptr, URestrictedGameplayTagsList* InSourceRestrictedTagList = nullptr) 
		: SourceName(InSourceName), SourceType(InSourceType), SourceTagList(InSourceTagList), SourceRestrictedTagList(InSourceRestrictedTagList)
	{
	}

	/** Returns the config file that created this source, if valid */
	GAMEPLAYTAGS_API FString GetConfigFileName() const;

	static GAMEPLAYTAGS_API FName GetNativeName();

	static GAMEPLAYTAGS_API FName GetDefaultName();

#if WITH_EDITOR
	static GAMEPLAYTAGS_API FName GetFavoriteName();

	static GAMEPLAYTAGS_API void SetFavoriteName(FName TagSourceToFavorite);

	static GAMEPLAYTAGS_API FName GetTransientEditorName();
#endif
};

/** Struct describing the places to look for ini search paths */
struct FGameplayTagSearchPathInfo
{
	/** Which sources should be loaded from this path */
	TArray<FName> SourcesInPath;

	/** Config files to load from, will normally correspond to FoundSources */
	TArray<FString> TagIniList;

	/** True if this path has already been searched */
	bool bWasSearched = false;

	/** True if the tags in sources have been added to the current tree */
	bool bWasAddedToTree = false;

	inline void Reset()
	{
		SourcesInPath.Reset();
		TagIniList.Reset();
		bWasSearched = false;
		bWasAddedToTree = false;
	}

	inline bool IsValid()
	{
		return bWasSearched && bWasAddedToTree;
	}
};

/** Simple tree node for gameplay tags, this stores metadata about specific tags */
USTRUCT()
struct FGameplayTagNode
{
	GENERATED_USTRUCT_BODY()
	FGameplayTagNode(){};

	/** Simple constructor, passing redundant data for performance */
	FGameplayTagNode(FName InTag, FName InFullTag, TSharedPtr<FGameplayTagNode> InParentNode, bool InIsExplicitTag, bool InIsRestrictedTag, bool InAllowNonRestrictedChildren);

	/** Returns a correctly constructed container with only this tag, useful for doing container queries */
	inline const FGameplayTagContainer& GetSingleTagContainer() const { return CompleteTagWithParents; }

	/**
	 * Get the complete tag for the node, including all parent tags, delimited by periods
	 * 
	 * @return Complete tag for the node
	 */
	inline const FGameplayTag& GetCompleteTag() const { return CompleteTagWithParents.Num() > 0 ? CompleteTagWithParents.GameplayTags[0] : FGameplayTag::EmptyTag; }
	inline FName GetCompleteTagName() const { return GetCompleteTag().GetTagName(); }
	inline FString GetCompleteTagString() const { return GetCompleteTag().ToString(); }

	/**
	 * Get the simple tag for the node (doesn't include any parent tags)
	 * 
	 * @return Simple tag for the node
	 */
	inline FName GetSimpleTagName() const { return Tag; }

	/**
	 * Get the children nodes of this node
	 * 
	 * @return Reference to the array of the children nodes of this node
	 */
	inline TArray< TSharedPtr<FGameplayTagNode> >& GetChildTagNodes() { return ChildTags; }

	/**
	 * Get the children nodes of this node
	 * 
	 * @return Reference to the array of the children nodes of this node
	 */
	inline const TArray< TSharedPtr<FGameplayTagNode> >& GetChildTagNodes() const { return ChildTags; }

	/**
	 * Get the parent tag node of this node
	 * 
	 * @return The parent tag node of this node
	 */
	inline TSharedPtr<FGameplayTagNode> GetParentTagNode() const { return ParentNode; }

	/**
	* Get the net index of this node
	*
	* @return The net index of this node
	*/
	inline FGameplayTagNetIndex GetNetIndex() const { check(NetIndex != INVALID_TAGNETINDEX); return NetIndex; }

	/** Reset the node of all of its values */
	GAMEPLAYTAGS_API void ResetNode();

	/** Returns true if the tag was explicitly specified in code or data */
	inline bool IsExplicitTag() const {
#if WITH_EDITORONLY_DATA
		return bIsExplicitTag;
#else
		return true;
#endif
	}

	/** Returns true if the tag is a restricted tag and allows non-restricted children */
	inline bool GetAllowNonRestrictedChildren() const { 
#if WITH_EDITORONLY_DATA
		return bAllowNonRestrictedChildren;  
#else
		return true;
#endif
	}

	/** Returns true if the tag is a restricted tag */
	inline bool IsRestrictedGameplayTag() const {
#if WITH_EDITORONLY_DATA
		return bIsRestrictedTag;
#else
		return true;
#endif
	}

#if WITH_EDITORONLY_DATA
	FName GetFirstSourceName() const { return SourceNames.Num() == 0 ? NAME_None : SourceNames[0]; }
	const TArray<FName>& GetAllSourceNames() const { return SourceNames; }
#endif

#if WITH_EDITORONLY_DATA
	/** Returns the Comment for this tag */
	FString GetDevComment() const { return DevComment; }
#endif

#if WITH_EDITOR
	/**
	 * Update the hasher with a deterministic hash of the data on this. Used for e.g. IncrementalCook keys.
	 * Does not include data from this node's child or parent nodes.
	 */
	GAMEPLAYTAGS_API void Hash(FBlake3& Hasher);
#endif

private:
	/** Raw name for this tag at current rank in the tree */
	FName Tag;

	/** This complete tag is at GameplayTags[0], with parents in ParentTags[] */
	FGameplayTagContainer CompleteTagWithParents;

	/** Child gameplay tag nodes */
	TArray< TSharedPtr<FGameplayTagNode> > ChildTags;

	/** Owner gameplay tag node, if any */
	TSharedPtr<FGameplayTagNode> ParentNode;
	
	/** Net Index of this node */
	FGameplayTagNetIndex NetIndex;

#if WITH_EDITORONLY_DATA
	/** Module or Package or config file this tag came from. If empty this is an implicitly added tag */
	TArray<FName> SourceNames;

	/** Comment for this tag */
	FString DevComment;

	/** If this is true then the tag can only have normal tag children if bAllowNonRestrictedChildren is true */
	uint8 bIsRestrictedTag : 1;

	/** If this is true then any children of this tag must come from the restricted tags */
	uint8 bAllowNonRestrictedChildren : 1;

	/** If this is true then the tag was explicitly added and not only implied by its child tags */
	uint8 bIsExplicitTag : 1;

	/** If this is true then at least one tag that inherits from this tag is coming from multiple sources. Used for updating UI in the editor. */
	uint8 bDescendantHasConflict : 1;

	/** If this is true then this tag is coming from multiple sources. No descendants can be changed on this tag until this is resolved. */
	uint8 bNodeHasConflict : 1;

	/** If this is true then at least one tag that this tag descends from is coming from multiple sources. This tag and it's descendants can't be changed in the editor. */
	uint8 bAncestorHasConflict : 1;
#endif 

	friend class UGameplayTagsManager;
	friend class SGameplayTagWidget;
	friend class SGameplayTagPicker;
};

/** Holds data about the tag dictionary, is in a singleton UObject */
UCLASS(config=Engine, MinimalAPI)
class UGameplayTagsManager : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Destructor */
	GAMEPLAYTAGS_API ~UGameplayTagsManager();

	/** Returns the global UGameplayTagsManager manager */
	inline static UGameplayTagsManager& Get()
	{
		if (SingletonManager == nullptr)
		{
			InitializeManager();
		}

		return *SingletonManager;
	}

	/** Returns possibly nullptr to the manager. Needed for some shutdown cases to avoid reallocating. */
	inline static UGameplayTagsManager* GetIfAllocated() { return SingletonManager; }

	/**
	* Adds the gameplay tags corresponding to the strings in the array TagStrings to OutTagsContainer
	*
	* @param TagStrings Array of strings to search for as tags to add to the tag container
	* @param OutTagsContainer Container to add the found tags to.
	* @param ErrorIfNotfound: ensure() that tags exists.
	*
	*/
	GAMEPLAYTAGS_API void RequestGameplayTagContainer(const TArray<FString>& TagStrings, FGameplayTagContainer& OutTagsContainer, bool bErrorIfNotFound=true) const;

	/**
	 * Gets the FGameplayTag that corresponds to the TagName
	 *
	 * @param TagName The Name of the tag to search for
	 * @param ErrorIfNotfound: ensure() that tag exists.
	 * 
	 * @return Will return the corresponding FGameplayTag or an empty one if not found.
	 */
	GAMEPLAYTAGS_API FGameplayTag RequestGameplayTag(FName TagName, bool ErrorIfNotFound=true) const;

	/** 
	 * Returns true if this is a valid gameplay tag string (foo.bar.baz). If false, it will fill 
	 * @param TagString String to check for validity
	 * @param OutError If non-null and string invalid, will fill in with an error message
	 * @param OutFixedString If non-null and string invalid, will attempt to fix. Will be empty if no fix is possible
	 * @return True if this can be added to the tag dictionary, false if there's a syntax error
	 */
	GAMEPLAYTAGS_API bool IsValidGameplayTagString(const TCHAR* TagString, FText* OutError = nullptr, FString* OutFixedString = nullptr);
	GAMEPLAYTAGS_API bool IsValidGameplayTagString(const FString& TagString, FText* OutError = nullptr, FString* OutFixedString = nullptr);
	GAMEPLAYTAGS_API bool IsValidGameplayTagString(const FStringView& TagString, FText* OutError = nullptr, FStringBuilderBase* OutFixedString = nullptr);

	/**
	 *	Searches for a gameplay tag given a partial string. This is slow and intended mainly for console commands/utilities to make
	 *	developer life's easier. This will attempt to match as best as it can. If you pass "A.b" it will match on "A.b." before it matches "a.b.c".
	 */
	GAMEPLAYTAGS_API FGameplayTag FindGameplayTagFromPartialString_Slow(FString PartialString) const;

	/**
	 * Registers the given name as a gameplay tag, and tracks that it is being directly referenced from code
	 * This can be called after initial engine startup but can invalidate FastReplication
	 *
	 * @param TagName The Name of the tag to add
	 * @param TagDevComment The developer comment clarifying the usage of the tag
	 * 
	 * @return Will return the corresponding FGameplayTag
	 */
	GAMEPLAYTAGS_API FGameplayTag AddNativeGameplayTag(FName TagName, const FString& TagDevComment = TEXT("(Native)"));

private:
	// Only callable from FNativeGameplayTag, these functions do less error checking and can happen after initial tag loading is done
	GAMEPLAYTAGS_API void AddNativeGameplayTag(FNativeGameplayTag* TagSource);
	GAMEPLAYTAGS_API void RemoveNativeGameplayTag(const FNativeGameplayTag* TagSource);

public:
	/** Call to flush the list of native tags, once called it is unsafe to add more */
	GAMEPLAYTAGS_API void DoneAddingNativeTags();

	/** This is a delegate that is called during initialization/initial loading and signals the last chance to add tags before we are considered to be fully loaded (all tags registered). */
	UE_DEPRECATED(5.8, "Replace with CallOrRegister_OnAddNativeTagsDelegate or call AddNativeGameplayTag directly, tags can be safely added after startup with default settings")
	static GAMEPLAYTAGS_API FSimpleMulticastDelegate& OnLastChanceToAddNativeTags();

	/**
	 * Register a callback for initialization to add additional native tags right before fully initializing gameplay tags.
	 * Or, if the startup native tags have already been added, execute Delegate before the end of the current module load.
	 */
	static GAMEPLAYTAGS_API FDelegateHandle CallOrRegister_OnAddNativeTagsDelegate(const FSimpleMulticastDelegate::FDelegate& Delegate);

	/**
	 * Register a callback for when native tags are done being added (this is also a safe point to consider that the gameplay tags have fully been initialized).
	 * Or, if the native tags have already been added (and thus we have registered all valid tags), then execute this Delegate immediately.
	 * This is useful if your code is potentially executed during load time, and therefore any tags in your block of code could be not-yet-loaded, but possibly valid after being loaded.
	 */
	static GAMEPLAYTAGS_API FDelegateHandle CallOrRegister_OnDoneAddingNativeTagsDelegate(const FSimpleMulticastDelegate::FDelegate& Delegate);

	/** 
	 * Unregisters a previously registered callback from either OnAddNativeTagsDelegate or OnDoneAddingNativeTagsDelegate.
	 * If the delegate has already activated or the handle is empty this will do nothing.
	 */ 
	static GAMEPLAYTAGS_API void UnregisterNativeTagDelegate(FDelegateHandle DelegateHandle);

	/**
	 * Gets a Tag Container containing the supplied tag and all of its parents as explicit tags.
	 * For example, passing in x.y.z would return a tag container with x.y.z, x.y, and x.
	 * This will only work for tags that have been properly registered.
	 *
	 * @param GameplayTag The tag to use at the child most tag for this container
	 * 
	 * @return A tag container with the supplied tag and all its parents added explicitly, or an empty container if that failed
	 */
	GAMEPLAYTAGS_API FGameplayTagContainer RequestGameplayTagParents(const FGameplayTag& GameplayTag) const;

	/**
	 * Fills in an array of gameplay tags with all of tags that are the parents of the passed in tag.
	 * For example, passing in x.y.z would add x.y and x to UniqueParentTags if they was not already there.
	 * This is used by the GameplayTagContainer code and may work for unregistered tags depending on serialization settings.
	 *
	 * @param GameplayTag The gameplay tag to extract parent tags from
	 * @param UniqueParentTags A list of parent tags that will be added to if necessary
	 *
	 * @return true if any tags were added to UniqueParentTags
	 */
	GAMEPLAYTAGS_API bool ExtractParentTags(const FGameplayTag& GameplayTag, TArray<FGameplayTag>& UniqueParentTags) const;

	/**
	 * Gets a Tag Container containing the all tags in the hierarchy that are children of this tag. Does not return the original tag
	 *
	 * @param GameplayTag					The Tag to use at the parent tag
	 * 
	 * @return A Tag Container with the supplied tag and all its parents added explicitly
	 */
	GAMEPLAYTAGS_API FGameplayTagContainer RequestGameplayTagChildren(const FGameplayTag& GameplayTag) const;

	/** Returns direct parent GameplayTag of this GameplayTag, calling on x.y will return x */
	GAMEPLAYTAGS_API FGameplayTag RequestGameplayTagDirectParent(const FGameplayTag& GameplayTag) const;

	UE_DEPRECATED(5.4, "This function is not threadsafe, use FindTagNode or FGameplayTag::GetSingleTagContainer")
	inline const FGameplayTagContainer* GetSingleTagContainer(const FGameplayTag& GameplayTag) const
	{
		return GetSingleTagContainerPtr(GameplayTag);
	}

	/**
	 * Checks node tree to see if a FGameplayTagNode with the tag exists
	 *
	 * @param TagName	The name of the tag node to search for
	 *
	 * @return A shared pointer to the FGameplayTagNode found, or NULL if not found.
	 */
	inline TSharedPtr<FGameplayTagNode> FindTagNode(const FGameplayTag& GameplayTag) const
	{
		UE::TScopeLock Lock(GameplayTagMapCritical);

		const TSharedPtr<FGameplayTagNode>* Node = GameplayTagNodeMap.Find(GameplayTag);

		if (Node)
		{
			return *Node;
		}
#if WITH_EDITOR
		// Check redirector
		if (GIsEditor && GameplayTag.IsValid())
		{
			FGameplayTag RedirectedTag = GameplayTag;

			RedirectSingleGameplayTag(RedirectedTag, nullptr);

			Node = GameplayTagNodeMap.Find(RedirectedTag);

			if (Node)
			{
				return *Node;
			}
		}
#endif
		return nullptr;
	}

	/**
	 * Checks node tree to see if a FGameplayTagNode with the name exists
	 *
	 * @param TagName	The name of the tag node to search for
	 *
	 * @return A shared pointer to the FGameplayTagNode found, or NULL if not found.
	 */
	inline TSharedPtr<FGameplayTagNode> FindTagNode(FName TagName) const
	{
		FGameplayTag PossibleTag(TagName);
		return FindTagNode(PossibleTag);
	}

	/** Loads the tag tables referenced in the GameplayTagSettings object */
	GAMEPLAYTAGS_API void LoadGameplayTagTables(bool bAllowAsyncLoad = false);

	/** Loads tag inis contained in the specified path, passes an optional PluginConfigCache to speed up disk searches */
	GAMEPLAYTAGS_API void AddTagIniSearchPath(const FString& RootDir, const TSet<FString>* PluginConfigsCache = nullptr);

	/** Tries to remove the specified search path, will return true if anything was removed */
	GAMEPLAYTAGS_API bool RemoveTagIniSearchPath(const FString& RootDir);

	/** Gets all the current directories to look for tag sources in */
	GAMEPLAYTAGS_API void GetTagSourceSearchPaths(TArray<FString>& OutPaths);

	/** Gets the number of tag source search paths */
	GAMEPLAYTAGS_API int32 GetNumTagSourceSearchPaths();

	/** Helper function to construct the gameplay tag tree */
	GAMEPLAYTAGS_API void ConstructGameplayTagTree();

	/** Helper function to destroy the gameplay tag tree */
	GAMEPLAYTAGS_API void DestroyGameplayTagTree();

	/** Splits a tag such as x.y.z into an array of names {x,y,z} */
	GAMEPLAYTAGS_API void SplitGameplayTagFName(const FGameplayTag& Tag, TArray<FName>& OutNames) const;

	/** Gets the list of all registered tags, setting OnlyIncludeDictionaryTags will exclude implicitly added tags if possible */
	GAMEPLAYTAGS_API void RequestAllGameplayTags(FGameplayTagContainer& TagContainer, bool OnlyIncludeDictionaryTags) const;

	/** Returns true if if the passed in name is in the tag dictionary and can be created */
	GAMEPLAYTAGS_API bool ValidateTagCreation(FName TagName) const;

	/** Returns the tag source for a given tag source name and type, or null if not found */
	GAMEPLAYTAGS_API const FGameplayTagSource* FindTagSource(FName TagSourceName) const;

	/** Returns the tag source for a given tag source name and type, or null if not found */
	GAMEPLAYTAGS_API FGameplayTagSource* FindTagSource(FName TagSourceName);

	/** Fills in an array with all tag sources of a specific type */
	GAMEPLAYTAGS_API void FindTagSourcesWithType(EGameplayTagSourceType TagSourceType, TArray<const FGameplayTagSource*>& OutArray) const;

	GAMEPLAYTAGS_API void FindTagsWithSource(FStringView PackageNameOrPath, TArray<FGameplayTag>& OutTags) const;

	/**
	 * Check to see how closely two FGameplayTags match. Higher values indicate more matching terms in the tags.
	 *
	 * @param GameplayTagOne	The first tag to compare
	 * @param GameplayTagTwo	The second tag to compare
	 *
	 * @return the length of the longest matching pair
	 */
	GAMEPLAYTAGS_API int32 GameplayTagsMatchDepth(const FGameplayTag& GameplayTagOne, const FGameplayTag& GameplayTagTwo) const;

	/** Returns the number of parents a particular gameplay tag has.  Useful as a quick way to determine which tags may
	 * be more "specific" than other tags without comparing whether they are in the same hierarchy or anything else.
	 * Example: "TagA.SubTagA" has 2 Tag Nodes.  "TagA.SubTagA.LeafTagA" has 3 Tag Nodes.
	 */ 
	GAMEPLAYTAGS_API int32 GetNumberOfTagNodes(const FGameplayTag& GameplayTag) const;

	/** Returns true if we should import tags from UGameplayTagsSettings objects (configured by INI files) */
	GAMEPLAYTAGS_API bool ShouldImportTagsFromINI() const;

	/** Should we print loading errors when trying to load invalid tags */
	bool ShouldWarnOnInvalidTags() const
	{
		return bShouldWarnOnInvalidTags;
	}

	/** Should we clear references to invalid tags loaded/saved in the editor */
	UE_DEPRECATED(5.5, "We should never clear invalid tags as we're not guaranteed the required plugin has loaded")
	bool ShouldClearInvalidTags() const
	{
		return false;
	}

	/** Should use fast replication */
	bool ShouldUseFastReplication() const
	{
		return bUseFastReplication;
	}

	/** Should use dynamic replication (Gameplay Tags need not match between client/server) */
	bool ShouldUseDynamicReplication() const
	{
		return !bUseFastReplication && bUseDynamicReplication;
	}

	/** If we are allowed to unload tags */
	GAMEPLAYTAGS_API bool ShouldUnloadTags() const;

	/** Pushes an override that supersedes bShouldAllowUnloadingTags to allow/disallow unloading of GameplayTags in controlled scenarios */
	GAMEPLAYTAGS_API void SetShouldUnloadTagsOverride(bool bShouldUnloadTags);

	/** Clears runtime overrides, reverting to bShouldAllowUnloadingTags when determining GameplayTags unload behavior */
	GAMEPLAYTAGS_API void ClearShouldUnloadTagsOverride();

	/** Pushes an override that suppresses calls to HandleGameplayTagTreeChanged that would result in a complete rebuild of the GameplayTag tree */
	GAMEPLAYTAGS_API void SetShouldDeferGameplayTagTreeRebuilds(bool bShouldDeferRebuilds);

	/** Stops suppressing GameplayTag tree rebuilds and (optionally) rebuilds the tree */
	GAMEPLAYTAGS_API void ClearShouldDeferGameplayTagTreeRebuilds(bool bRebuildTree);

	/** Temporarily disables IGameplayTagsModule::OnGameplayTagTreeChanged callback for changes to the gameplay tag tree. This does not stop it from actually rebuilding the tree */
	GAMEPLAYTAGS_API void PushDeferOnGameplayTagTreeChangedBroadcast();

	/** Re-enables OnGameplayTagTreeChanged callbacks, there must be one call to this function for every Push. This will execute callbacks immediately if they were deferred */
	GAMEPLAYTAGS_API void PopDeferOnGameplayTagTreeChangedBroadcast();

	/** Returns the hash of NetworkGameplayTagNodeIndex */
	uint32 GetNetworkGameplayTagNodeIndexHash() const { VerifyNetworkIndex(); return NetworkGameplayTagNodeIndexHash; }

	/** Returns a list of the ini files that contain restricted tags */
	GAMEPLAYTAGS_API void GetRestrictedTagConfigFiles(TArray<FString>& RestrictedConfigFiles) const;

	/** Returns a list of the source files that contain restricted tags */
	GAMEPLAYTAGS_API void GetRestrictedTagSources(TArray<const FGameplayTagSource*>& Sources) const;

	/** Returns a list of the owners for a restricted tag config file. May be empty */
	GAMEPLAYTAGS_API void GetOwnersForTagSource(const FString& SourceName, TArray<FString>& OutOwners) const;

	/** Notification that a tag container has been loaded via serialize */
	GAMEPLAYTAGS_API void GameplayTagContainerLoaded(FGameplayTagContainer& Container, FProperty* SerializingProperty) const;

	/** Notification that a gameplay tag has been loaded via serialize */
	GAMEPLAYTAGS_API void SingleGameplayTagLoaded(FGameplayTag& Tag, FProperty* SerializingProperty) const;

	/** Handles redirectors for an entire container, will also error on invalid tags */
	GAMEPLAYTAGS_API void RedirectTagsForContainer(FGameplayTagContainer& Container, FProperty* SerializingProperty) const;

	/** Handles redirectors for a single tag, will also error on invalid tag. This is only called for when individual tags are serialized on their own */
	GAMEPLAYTAGS_API void RedirectSingleGameplayTag(FGameplayTag& Tag, FProperty* SerializingProperty) const;

	/** Handles establishing a single tag from an imported tag name (accounts for redirects too). Called when tags are imported via text. */
	GAMEPLAYTAGS_API bool ImportSingleGameplayTag(FGameplayTag& Tag, FName ImportedTagName, bool bImportFromSerialize = false) const;

	/** Gets a tag name from net index and vice versa, used for replication efficiency */
	GAMEPLAYTAGS_API FName GetTagNameFromNetIndex(FGameplayTagNetIndex Index) const;
	GAMEPLAYTAGS_API FGameplayTagNetIndex GetNetIndexFromTag(const FGameplayTag &InTag) const;

	/** Cached number of bits we need to replicate tags. That is, Log2(Number of Tags). Will always be <= 16. */
	int32 GetNetIndexTrueBitNum() const { VerifyNetworkIndex(); return NetIndexTrueBitNum; }

	/** The length in bits of the first segment when net serializing tags. We will serialize NetIndexFirstBitSegment + 1 bit to indicatore "more" (more = second segment that is NetIndexTrueBitNum - NetIndexFirstBitSegment) */
	int32 GetNetIndexFirstBitSegment() const { VerifyNetworkIndex(); return NetIndexFirstBitSegment; }

	/** This is the actual value for an invalid tag "None". This is computed at runtime as (Total number of tags) + 1 */
	FGameplayTagNetIndex GetInvalidTagNetIndex() const { VerifyNetworkIndex(); return InvalidTagNetIndex; }

	const TArray<TSharedPtr<FGameplayTagNode>>& GetNetworkGameplayTagNodeIndex() const { VerifyNetworkIndex(); return NetworkGameplayTagNodeIndex; }

	DECLARE_TS_MULTICAST_DELEGATE_OneParam(FOnGameplayTagLoaded, const FGameplayTag& /*Tag*/)
	FOnGameplayTagLoaded OnGameplayTagLoadedDelegate;

	/** Numbers of bits to use for replicating container size. This can be set via config. */
	int32 NumBitsForContainerSize;

private:
	/** Cached number of bits we need to replicate tags. That is, Log2(Number of Tags). Will always be <= 16. */
	int32 NetIndexTrueBitNum;

	/** The length in bits of the first segment when net serializing tags. We will serialize NetIndexFirstBitSegment + 1 bit to indicatore "more" (more = second segment that is NetIndexTrueBitNum - NetIndexFirstBitSegment) */
	int32 NetIndexFirstBitSegment;

	/** This is the actual value for an invalid tag "None". This is computed at runtime as (Total number of tags) + 1 */
	FGameplayTagNetIndex InvalidTagNetIndex;

public:

#if WITH_EDITOR
	/** Gets a Filtered copy of the GameplayRootTags Array based on the comma delimited filter string passed in */
	GAMEPLAYTAGS_API void GetFilteredGameplayRootTags(const FString& InFilterString, TArray< TSharedPtr<FGameplayTagNode> >& OutTagArray) const;

	/** Returns "Categories" meta property from given handle, used for filtering by tag widget */
	GAMEPLAYTAGS_API FString GetCategoriesMetaFromPropertyHandle(TSharedPtr<class IPropertyHandle> PropertyHandle) const;

	/** Helper function, made to be called by custom OnGetCategoriesMetaFromPropertyHandle handlers  */
	static GAMEPLAYTAGS_API FString StaticGetCategoriesMetaFromPropertyHandle(TSharedPtr<class IPropertyHandle> PropertyHandle);

	/** Returns "Categories" meta property from given field, used for filtering by tag widget */
	template <typename TFieldType>
	static FString GetCategoriesMetaFromField(TFieldType* Field)
	{
		check(Field);
		if (Field->HasMetaData(NAME_Categories))
		{
			return Field->GetMetaData(NAME_Categories);
		}
		else if (Field->HasMetaData(NAME_GameplayTagFilter))
		{
			return Field->GetMetaData(NAME_GameplayTagFilter);
		}
		return FString();
	}

	/** Returns "GameplayTagFilter" meta property from given function, used for filtering by tag widget for any parameters of the function that end up as BP pins */
	static GAMEPLAYTAGS_API FString GetCategoriesMetaFromFunction(const UFunction* Func, FName ParamName = NAME_None);

	/** Gets a list of all gameplay tag nodes added by the specific source */
	GAMEPLAYTAGS_API void GetAllTagsFromSource(FName TagSource, TArray< TSharedPtr<FGameplayTagNode> >& OutTagArray) const;

	/** Returns true if this tag was explicitly registered, this is false for implictly added parent tags */
	GAMEPLAYTAGS_API bool IsDictionaryTag(FName TagName) const;

	/** Returns information about tag. If not found return false */
	GAMEPLAYTAGS_API bool GetTagEditorData(FName TagName, FString& OutComment, FName &OutFirstTagSource, bool& bOutIsTagExplicit, bool &bOutIsRestrictedTag, bool &bOutAllowNonRestrictedChildren) const;
	
	/** Returns information about tag. If not found return false */
    GAMEPLAYTAGS_API bool GetTagEditorData(FName TagName, FString& OutComment, TArray<FName>& OutTagSources, bool& bOutIsTagExplicit, bool &bOutIsRestrictedTag, bool &bOutAllowNonRestrictedChildren) const;

	/** 
	 * This is called after EditorRefreshGameplayTagTree. Useful if you need to do anything editor related when the gameplay tag tree is destroyed and recreated.
	 * IGameplayTagsModule::OnGameplayTagTreeChanged is called first and will happen when any tags are added or removed, not just when the settings are fully refreshed.
	 */
	static GAMEPLAYTAGS_API FSimpleMulticastDelegate OnEditorRefreshGameplayTagTree;

	/** Refresh the gameplaytag tree due to an editor change */
	GAMEPLAYTAGS_API void EditorRefreshGameplayTagTree();

	/** Suspends EditorRefreshGameplayTagTree requests */
	GAMEPLAYTAGS_API void SuspendEditorRefreshGameplayTagTree(FGuid SuspendToken);

	/** Resumes EditorRefreshGameplayTagTree requests; triggers a refresh if a request was made while it was suspended */
	GAMEPLAYTAGS_API void ResumeEditorRefreshGameplayTagTree(FGuid SuspendToken);

	/** Gets a Tag Container containing all of the tags in the hierarchy that are children of this tag, and were explicitly added to the dictionary */
	GAMEPLAYTAGS_API FGameplayTagContainer RequestGameplayTagChildrenInDictionary(const FGameplayTag& GameplayTag) const;
#if WITH_EDITORONLY_DATA
	/** Gets a Tag Container containing all of the tags in the hierarchy that are children of this tag, were explicitly added to the dictionary, and do not have any explicitly added tags between them and the specified tag */
	GAMEPLAYTAGS_API FGameplayTagContainer RequestGameplayTagDirectDescendantsInDictionary(const FGameplayTag& GameplayTag, EGameplayTagSelectionType SelectionType) const;
#endif // WITH_EDITORONLY_DATA


	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGameplayTagDoubleClickedEditor, FGameplayTag, FSimpleMulticastDelegate& /* OUT */)
	FOnGameplayTagDoubleClickedEditor OnGatherGameplayTagDoubleClickedEditor;

	/** Chance to dynamically change filter string based on a property handle */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGetCategoriesMetaFromPropertyHandle, TSharedPtr<IPropertyHandle>, FString& /* OUT */)
	FOnGetCategoriesMetaFromPropertyHandle OnGetCategoriesMetaFromPropertyHandle;

	/** Allows dynamic hiding of gameplay tags in SGameplayTagWidget. Allows higher order structs to dynamically change which tags are visible based on its own data */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnFilterGameplayTagChildren, const FString&  /** FilterString */, TSharedPtr<FGameplayTagNode>& /* TagNode */, bool& /* OUT OutShouldHide */)
	FOnFilterGameplayTagChildren OnFilterGameplayTagChildren;

	/*
	* This is a container to filter out gameplay tags when they are invalid or when they don't meet the filter string
	* If used from editor to filter out tags when picking them the FilterString is optional and the ReferencingPropertyHandle is required
	* If used to validate an asset / assets you can provide the TagSourceAssets. The FilterString and ReferencingPropertyHandle is optional
	*/
	struct FFilterGameplayTagContext
	{
		const FString& FilterString;
		const TSharedPtr<FGameplayTagNode>& TagNode;
		const FGameplayTagSource* TagSource;
		const TSharedPtr<IPropertyHandle> ReferencingPropertyHandle;
		const TArray<FAssetData> TagSourceAssets;

		FFilterGameplayTagContext(const FString& InFilterString, const TSharedPtr<FGameplayTagNode>& InTagNode, const FGameplayTagSource* InTagSource, const TSharedPtr<IPropertyHandle>& InReferencingPropertyHandle)
			: FilterString(InFilterString), TagNode(InTagNode), TagSource(InTagSource), ReferencingPropertyHandle(InReferencingPropertyHandle)
		{}

		FFilterGameplayTagContext(const TSharedPtr<FGameplayTagNode>& InTagNode, const FGameplayTagSource* InTagSource, const TArray<FAssetData>& InTagSourceAssets, const FString& InFilterString = FString())
			: FilterString(InFilterString), TagNode(InTagNode), TagSource(InTagSource), TagSourceAssets(InTagSourceAssets)
		{}
	};

	/*
	 * Allows dynamic hiding of gameplay tags in SGameplayTagWidget. Allows higher order structs to dynamically change which tags are visible based on its own data
	 * Applies to all tags, and has more context than OnFilterGameplayTagChildren
	 */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFilterGameplayTag, const FFilterGameplayTagContext& /** InContext */, bool& /* OUT OutShouldHide */)
	FOnFilterGameplayTag OnFilterGameplayTag;
	
	GAMEPLAYTAGS_API void NotifyGameplayTagDoubleClickedEditor(FString TagName);
	
	GAMEPLAYTAGS_API bool ShowGameplayTagAsHyperLinkEditor(FString TagName);

	/**
	 * Used for incremental cooking. Create an FCookDependency for a gameplay tag.
	 * Packages that pass this dependency to AddCookLoadDependency or AddCookSaveDependency in their OnCookEvent or
	 * (if Ar.IsCooking()) Serialize function will be invalidated and recooked by the incremental cook whenever those
	 * tags change.
	 */
	GAMEPLAYTAGS_API UE::Cook::FCookDependency CreateCookDependency(const FGameplayTag& Tag) const;

	GAMEPLAYTAGS_API void AppendDependencyHash(const FName& TagName, FBlake3& OutHasher) const;

	/** Implementation of console command GameplayTags.DumpSources */
	void DumpSources(FOutputDevice& Out) const;
#endif //WITH_EDITOR

	GAMEPLAYTAGS_API void PrintReplicationIndices();
	int32 GetNumGameplayTagNodes() const { return GameplayTagNodeMap.Num(); }

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	/** Mechanism for tracking what tags are frequently replicated */

	GAMEPLAYTAGS_API void PrintReplicationFrequencyReport();
	GAMEPLAYTAGS_API void NotifyTagReplicated(FGameplayTag Tag, bool WasInContainer);

	TMap<FGameplayTag, int32>	ReplicationCountMap;
	TMap<FGameplayTag, int32>	ReplicationCountMap_SingleTags;
	TMap<FGameplayTag, int32>	ReplicationCountMap_Containers;
#endif

private:

	/** Initializes the manager */
	static GAMEPLAYTAGS_API void InitializeManager();

	/** finished loading/adding native tags */
	static GAMEPLAYTAGS_API FSimpleMulticastDelegate& OnDoneAddingNativeTagsDelegate();

	/** The Tag Manager singleton */
	static GAMEPLAYTAGS_API UGameplayTagsManager* SingletonManager;

	friend class FGameplayTagTest;
	friend class FGameplayEffectsTest;
	friend class FGameplayTagsModule;
	friend class FGameplayTagsEditorModule;
	friend class UGameplayTagsSettings;
	friend class SAddNewGameplayTagSourceWidget;
	friend class FNativeGameplayTag;

	/**
	 * Helper function to get the stored TagContainer containing only this tag, which has searchable ParentTags
	 * NOTE: This function is not threadsafe and should only be used in code that locks the tag map critical section
	 * @param GameplayTag		Tag to get single container of
	 * @return					Pointer to container with this tag
	 */
	inline const FGameplayTagContainer* GetSingleTagContainerPtr(const FGameplayTag& GameplayTag) const
	{
		// Doing this with pointers to avoid a shared ptr reference count change
		const TSharedPtr<FGameplayTagNode>* Node = GameplayTagNodeMap.Find(GameplayTag);

		if (Node)
		{
			return &(*Node)->GetSingleTagContainer();
		}
#if WITH_EDITOR
		// Check redirector
		if (GIsEditor && GameplayTag.IsValid())
		{
			FGameplayTag RedirectedTag = GameplayTag;

			RedirectSingleGameplayTag(RedirectedTag, nullptr);

			Node = GameplayTagNodeMap.Find(RedirectedTag);

			if (Node)
			{
				return &(*Node)->GetSingleTagContainer();
			}
		}
#endif
		return nullptr;
	}


	/**
	 * Helper function to insert a tag into a tag node array
	 *
	 * @param Tag							Short name of tag to insert
	 * @param FullTag						Full tag, passed in for performance
	 * @param ParentNode					Parent node, if any, for the tag
	 * @param NodeArray						Node array to insert the new node into, if necessary (if the tag already exists, no insertion will occur)
	 * @param SourceName					File tag was added from
	 * @param DevComment					Comment from developer about this tag
	 * @param bIsExplicitTag				Is the tag explicitly defined or is it implied by the existence of a child tag
	 * @param bIsRestrictedTag				Is the tag a restricted tag or a regular gameplay tag
	 * @param bAllowNonRestrictedChildren	If the tag is a restricted tag, can it have regular gameplay tag children or should all of its children be restricted tags as well?
	 *
	 * @return Index of the node of the tag
	 */
	GAMEPLAYTAGS_API int32 InsertTagIntoNodeArray(FName Tag, FName FullTag, TSharedPtr<FGameplayTagNode> ParentNode, TArray< TSharedPtr<FGameplayTagNode> >& NodeArray, FName SourceName, const FString& DevComment, bool bIsExplicitTag, bool bIsRestrictedTag, bool bAllowNonRestrictedChildren);

	/** Helper function to populate the tag tree from each table */
	GAMEPLAYTAGS_API void PopulateTreeFromDataTable(class UDataTable* Table);

	GAMEPLAYTAGS_API void AddTagTableRow(const FGameplayTagTableRow& TagRow, FName SourceName, bool bIsRestrictedTag = false);

	GAMEPLAYTAGS_API void AddChildrenTags(FGameplayTagContainer& TagContainer, TSharedPtr<FGameplayTagNode> GameplayTagNode, bool RecurseAll=true, bool OnlyIncludeDictionaryTags=false) const;

	GAMEPLAYTAGS_API void AddRestrictedGameplayTagSource(const FString& FileName);

	GAMEPLAYTAGS_API void AddTagsFromAdditionalLooseIniFiles(const TArray<FString>& IniFileList);

	/**
	 * Helper function for GameplayTagsMatch to get all parents when doing a parent match,
	 * NOTE: Must never be made public as it uses the FNames which should never be exposed
	 * 
	 * @param NameList		The list we are adding all parent complete names too
	 * @param GameplayTag	The current Tag we are adding to the list
	 */
	GAMEPLAYTAGS_API void GetAllParentNodeNames(TSet<FName>& NamesList, TSharedPtr<FGameplayTagNode> GameplayTag) const;

	/** Returns the tag source for a given tag source name, or null if not found */
	GAMEPLAYTAGS_API FGameplayTagSource* FindOrAddTagSource(FName TagSourceName, EGameplayTagSourceType SourceType, const FString& RootDirToUse = FString());

	/** Constructs the net indices for each tag */
	GAMEPLAYTAGS_API void ConstructNetIndex();

	/** Marks all of the nodes that descend from CurNode as having an ancestor node that has a source conflict. */
	GAMEPLAYTAGS_API void MarkChildrenOfNodeConflict(TSharedPtr<FGameplayTagNode> CurNode);

	void VerifyNetworkIndex() const
	{
		if (!bUseFastReplication)
		{
			UE_LOGF(LogGameplayTags, Warning, "%s called when not using FastReplication (not rebuilding the fast replication cache)", __func__);
		}
		else if (bNetworkIndexInvalidated)
		{
			const_cast<UGameplayTagsManager*>(this)->ConstructNetIndex();
		}
	}

	void InvalidateNetworkIndex() { bNetworkIndexInvalidated = true; }

	/** Called in both editor and game when the tag tree changes during startup or editing */
	GAMEPLAYTAGS_API void BroadcastOnGameplayTagTreeChanged();

	/** Call after modifying the tag tree nodes, this will either call the full editor refresh or a limited game refresh */
	GAMEPLAYTAGS_API void HandleGameplayTagTreeChanged(bool bRecreateTree);

	/** Handles pending tag changes */
	void HandleModuleObjectsLoaded(FName InModuleName, bool bCanProcessNewlyLoadedObjects);
	
	/** Temporarily disables any game or editor rebuilds of the gameplay tag tree, this should only be used when making multiple tag changes at once */
	void PushDeferGameplayTagTreeRebuilds();

	/** Re-enables game and editor rebuilds of the gameplay tag tree, there must be one call to this function for every Push. This will rebuild the tree immediately if it was deferred */
	void PopDeferGameplayTagTreeRebuilds();

#if WITH_EDITOR

	GameplayTagUtil::FGameplayTagCookDependency CreateGameplayTagCookDependency(const FGameplayTag& Tag) const;
	
	void ValidateGameplayTagCookDependencies(const FCbFieldViewIterator& Args, UE::Cook::FCookDependencyContext& Context) const;
#endif //#if WITH_EDITOR

	// Tag Sources
	///////////////////////////////////////////////////////

	/** These are the old native tags that use to be resisted via a function call with no specific site/ownership. */
	TSet<FName> LegacyNativeTags;

	/** Map of all config directories to load tag inis from */
	TMap<FString, FGameplayTagSearchPathInfo> RegisteredSearchPaths;

	/** Roots of gameplay tag nodes */
	TSharedPtr<FGameplayTagNode> GameplayRootTag;

	/** Map of Tags to Nodes - Internal use only. FGameplayTag is inside node structure, do not use FindKey! */
	TMap<FGameplayTag, TSharedPtr<FGameplayTagNode>> GameplayTagNodeMap;

	/** Our aggregated, sorted list of commonly replicated tags. These tags are given lower indices to ensure they replicate in the first bit segment. */
	TArray<FGameplayTag> CommonlyReplicatedTags;

	/** Map of gameplay tag source names to source objects */
	UPROPERTY()
	TMap<FName, FGameplayTagSource> TagSources;

	TSet<FName> RestrictedGameplayTagSourceNames;

	bool bIsConstructingGameplayTagTree = false;

	/** Cached runtime value for whether we are using fast replication or not. Initialized from config setting. */
	bool bUseFastReplication;

	/** Cached runtime value for whether we are using dynamic replication or not. Initialized from the config setting. */
	bool bUseDynamicReplication;

	/** Cached runtime value for whether we should warn when loading invalid tags */
	bool bShouldWarnOnInvalidTags;

	/** Cached runtime value for whether we should allow unloading of tags */
	bool bShouldAllowUnloadingTags;

	/** Augments usage of bShouldAllowUnloadingTags to allow runtime overrides to allow/disallow unloading of GameplayTags in controlled scenarios */
	TOptional<bool> ShouldAllowUnloadingTagsOverride;

	/** Used to suppress calls to HandleGameplayTagTreeChanged that would result in a complete rebuild of the GameplayTag tree*/
	TOptional<bool> ShouldDeferGameplayTagTreeRebuilds;

	/** Used for HandleModuleObjectsLoaded */
	FDelegateHandle HandleModuleObjectsLoadedHandle;

	/** True if native tags have all been added and flushed */
	bool bDoneAddingNativeTags;

	/** Used to suppress calls to BroadcastOnGameplayTagTreeChanged*/
	int32 DeferBroadcastOnGameplayTagTreeChanged = 0;
	bool bBroadcastOnGameplayTagTreeChangedWasDeferred = false;

	/** Used to suppress calls to HandleGameplayTagTreeChanged that would result in a complete rebuild of the GameplayTag tree */
	int32 DeferGameplayTagTreeRebuilds = 0;
	bool bGameplayTagTreeRebuildWasDeferred = false;

	/** If true, an action that would require a tree rebuild was performed during initialization **/
	bool bNeedsTreeRebuildOnDoneAddingGameplayTags = false;

	/** String with outlawed characters inside tags */
	FString InvalidTagCharacters;

	// This critical section is to handle an issue where tag requests come from another thread when async loading from a background thread in FGameplayTagContainer::Serialize.
	// This class is not generically threadsafe, but this should be locked by any operation that could update something read by a background thread.
	mutable FTransactionallySafeCriticalSection GameplayTagMapCritical;

#if WITH_EDITOR
	// Transient editor-only tags to support quick-iteration PIE workflows
	TSet<FName> TransientEditorTags;

	FBlake3Hash IncrementalCookHash;
#endif //if WITH_EDITOR

	/** Sorted list of nodes, used for network replication */
	TArray<TSharedPtr<FGameplayTagNode>> NetworkGameplayTagNodeIndex;

	uint32 NetworkGameplayTagNodeIndexHash;

	bool bNetworkIndexInvalidated = true;

	/** Holds all of the valid gameplay-related tags that can be applied to assets */
	UPROPERTY()
	TArray<TObjectPtr<UDataTable>> GameplayTagTables;

	GAMEPLAYTAGS_API const static FName NAME_Categories;
	GAMEPLAYTAGS_API const static FName NAME_GameplayTagFilter;

	friend class UGameplayTagsManagerIncrementalCookFunctions;

	mutable TLruCache<FGameplayTag, FGameplayTagContainer> ChildNodesCache;
};
