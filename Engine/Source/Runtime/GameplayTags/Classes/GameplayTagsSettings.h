// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "GameplayTagsManager.h"
#include "Engine/DeveloperSettings.h"
#include "GameplayTagRedirectors.h"
#include "GameplayTagsSettings.generated.h"

/** Category remapping. This allows base engine tag category meta data to remap to multiple project-specific categories. */
USTRUCT()
struct FGameplayTagCategoryRemap
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(EditAnywhere, Category = GameplayTags)
	FString BaseCategory;

	UPROPERTY(EditAnywhere, Category = GameplayTags)
	TArray<FString> RemapCategories;

	friend inline bool operator==(const FGameplayTagCategoryRemap& A, const FGameplayTagCategoryRemap& B)
	{
		return A.BaseCategory == B.BaseCategory && A.RemapCategories == B.RemapCategories;
	}
};

/** Base class for storing a list of gameplay tags as an ini list. This is used for both the central list and additional lists */
UCLASS(config = GameplayTagsList, MinimalAPI)
class UGameplayTagsList : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Relative path to the ini file that is backing this list */
	UPROPERTY()
	FString ConfigFileName;

	/** List of active tag redirects */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags, meta = (ConfigRestartRequired = true))
	TArray<FGameplayTagRedirect> GameplayTagRedirects;

	/** List of tags saved to this file */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags)
	TArray<FGameplayTagTableRow> GameplayTagList;

	/** Sorts tags alphabetically */
	GAMEPLAYTAGS_API void SortTags();
};

/** Base class for storing a list of restricted gameplay tags as an ini list. This is used for both the central list and additional lists */
UCLASS(config = GameplayTags, MinimalAPI)
class URestrictedGameplayTagsList : public UObject
{
	GENERATED_UCLASS_BODY()

	/** Relative path to the ini file that is backing this list */
	UPROPERTY()
	FString ConfigFileName;

	/** List of restricted tags saved to this file */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags)
	TArray<FRestrictedGameplayTagTableRow> RestrictedGameplayTagList;

	/** Sorts tags alphabetically */
	GAMEPLAYTAGS_API void SortTags();
};

USTRUCT()
struct FRestrictedConfigInfo
{
	GENERATED_BODY()

	/** Allows new tags to be saved into their own INI file. This is make merging easier for non technical developers by setting up their own ini file. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = GameplayTags)
	FString RestrictedConfigName;

	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = GameplayTags)
	TArray<FString> Owners;

	GAMEPLAYTAGS_API bool operator==(const FRestrictedConfigInfo& Other) const;
	GAMEPLAYTAGS_API bool operator!=(const FRestrictedConfigInfo& Other) const;
};

/**
 *	Class for importing GameplayTags directly from a config file.
 *	FGameplayTagsEditorModule::StartupModule adds this class to the Project Settings menu to be edited.
 *	Editing this in Project Settings will output changes to Config/DefaultGameplayTags.ini.
 *	
 *	Primary advantages of this approach are:
 *	-Adding new tags doesn't require checking out external and editing file (CSV or xls) then reimporting.
 *	-New tags are mergeable since .ini are text and non exclusive checkout.
 *	
 *	To do:
 *	-Better support could be added for adding new tags. We could match existing tags and autocomplete subtags as
 *	the user types (e.g, autocomplete 'Damage.Physical' as the user is adding a 'Damage.Physical.Slash' tag).
 *	
 */
UCLASS(config=GameplayTags, defaultconfig, MinimalAPI)
class UGameplayTagsSettings : public UGameplayTagsList
{
	GENERATED_UCLASS_BODY()

	/** If true, will import tags from ini files in the config/tags folder */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags)
	bool ImportTagsFromConfig;

	/** If true, will give load warnings when reading in saved tag references that are not in the dictionary */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags, meta = (ConfigRestartRequired = true))
	bool WarnOnInvalidTags;

	UE_DEPRECATED(5.5, "We never clear invalid tags when reading saved tag references as the loading order of native tags is not guaranteed.")
	UPROPERTY(config)
	bool ClearInvalidTags = false;

	/** If true, will allow unloading of tags in the editor when plugins are removed */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Gameplay Tags")
	bool AllowEditorTagUnloading;

	/** If true, will allow unloading of tags in a non-editor gebuild when plugins are removed, this is potentially unsafe and affects requests to unload during play in editor */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Gameplay Tags")
	bool AllowGameTagUnloading;

	/** If true, will replicate gameplay tags by index instead of name. For this to work, tags must be identical on client and server */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Replication")
	bool FastReplication;

	/** If true, will replicate gameplay tags dynamically by index per connection. Slightly higher cost than FastReplication, but tags can differ between client and server */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Replication", meta=(EditCondition="!FastReplication"))
	bool bDynamicReplication;

	/** These characters cannot be used in gameplay tags, in addition to special ones like newline*/
	UPROPERTY(config, EditAnywhere, Category = GameplayTags)
	FString InvalidTagCharacters;

	/** Category remapping. This allows base engine tag category meta data to remap to multiple project-specific categories. */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags)
	TArray<FGameplayTagCategoryRemap> CategoryRemapping;

	/** List of data tables to load tags from */
	UPROPERTY(config, EditAnywhere, Category = GameplayTags, meta = (AllowedClasses = "/Script/Engine.DataTable"))
	TArray<FSoftObjectPath> GameplayTagTableList;

	/** List of most frequently replicated tags */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Replication")
	TArray<FName> CommonlyReplicatedTags;

	/** Numbers of bits to use for replicating container size, set this based on how large your containers tend to be */
	UPROPERTY(config, EditAnywhere, Category = "Advanced Replication")
	int32 NumBitsForContainerSize;

	/** The length in bits of the first segment when net serializing tags. We will serialize NetIndexFirstBitSegment + 1 bit to indicate "more", which is slower to replicate */
	UPROPERTY(config, EditAnywhere, Category= "Advanced Replication")
	int32 NetIndexFirstBitSegment;

	/** A list of .ini files used to store restricted gameplay tags. */
	UPROPERTY(config, EditAnywhere, AdvancedDisplay, Category = "Advanced Gameplay Tags")
	TArray<FRestrictedConfigInfo> RestrictedConfigFiles;

#if WITH_EDITORONLY_DATA
	// Dummy parameters used to hook the editor UI
	/** Restricted Gameplay Tags.
	 * 
	 *  Restricted tags are intended to be top level tags that are important for your data hierarchy and modified by very few people.
	 */
	UPROPERTY(EditAnywhere, AdvancedDisplay, transient, Category = "Advanced Gameplay Tags")
	FString RestrictedTagList;

	/** Add a new gameplay tag config file for saving plugin or game-specific tags. */
	UPROPERTY(EditAnywhere, transient, Category = "GameplayTags")
	FString NewTagSource;

	/** Find and remove unused tags */
	UPROPERTY(EditAnywhere, transient, Category = "GameplayTags")
	FString CleanupUnusedTags;
#endif

#if WITH_EDITOR
	GAMEPLAYTAGS_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	GAMEPLAYTAGS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

private:
	// temporary copy of RestrictedConfigFiles that we use to identify changes in the list
	// this is required to autopopulate the owners field
	TArray<FRestrictedConfigInfo> RestrictedConfigFilesTempCopy;
#endif
};

UCLASS(config=EditorPerProjectUserSettings, meta=(DisplayName="Gameplay Tag Editing"), MinimalAPI)
class UGameplayTagsDeveloperSettings : public UDeveloperSettings
{
	GENERATED_UCLASS_BODY()

	GAMEPLAYTAGS_API virtual FName GetCategoryName() const override;

	/** Allows new tags to be saved into their own INI file. This is make merging easier for non technical developers by setting up their own ini file. */
	UPROPERTY(config, EditAnywhere, Category=GameplayTags)
	FString DeveloperConfigName;

	/** Stores the favorite tag source, used as the default ini when adding new tags, can be toggled on/off using the button next to the tag source picker */
	UPROPERTY(config, EditAnywhere, Category=GameplayTags)
	FName FavoriteTagSource;
};
