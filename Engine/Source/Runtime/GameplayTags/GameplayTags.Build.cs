// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayTags : ModuleRules
	{
		public GameplayTags(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"DeveloperSettings"
				}
				);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Projects",
					"NetCore",
					"Json",
					"JsonUtilities"
				}
			);

			if (Target.bCompileAgainstEditor)
            {
                PrivateDependencyModuleNames.AddRange(
                new string[]
                {
                    "SlateCore",
                    "Slate"
				}
                );
            }

			SetupIrisSupport(Target);
		}
	}
}
