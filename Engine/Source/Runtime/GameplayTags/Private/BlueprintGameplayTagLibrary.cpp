// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintGameplayTagLibrary.h"
#include "GameplayTagsModule.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BlueprintGameplayTagLibrary)

UBlueprintGameplayTagLibrary::UBlueprintGameplayTagLibrary(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool UBlueprintGameplayTagLibrary::MatchesTag(FGameplayTag TagOne, FGameplayTag TagTwo, bool bExactMatch)
{
	if (bExactMatch)
	{
		return TagOne.MatchesTagExact(TagTwo);
	}

	return TagOne.MatchesTag(TagTwo);
}

bool UBlueprintGameplayTagLibrary::MatchesAnyTags(FGameplayTag TagOne, const FGameplayTagContainer& OtherContainer, bool bExactMatch)
{
	if (bExactMatch)
	{
		return TagOne.MatchesAnyExact(OtherContainer);
	}
	return TagOne.MatchesAny(OtherContainer);
}

bool UBlueprintGameplayTagLibrary::EqualEqual_GameplayTag(FGameplayTag A, FGameplayTag B)
{
	return A == B;
}

bool UBlueprintGameplayTagLibrary::NotEqual_GameplayTag(FGameplayTag A, FGameplayTag B)
{
	return A != B;
}

bool UBlueprintGameplayTagLibrary::IsGameplayTagValid(FGameplayTag GameplayTag)
{
	return GameplayTag.IsValid();
}

FName UBlueprintGameplayTagLibrary::GetTagName(const FGameplayTag& GameplayTag)
{
	return GameplayTag.GetTagName();
}

FGameplayTag UBlueprintGameplayTagLibrary::MakeLiteralGameplayTag(FGameplayTag Value)
{
	return Value;
}

int32 UBlueprintGameplayTagLibrary::GetNumGameplayTagsInContainer(const FGameplayTagContainer& TagContainer)
{
	return TagContainer.Num();
}

bool UBlueprintGameplayTagLibrary::HasTag(const FGameplayTagContainer& TagContainer, FGameplayTag Tag, bool bExactMatch)
{
	if (bExactMatch)
	{
		return TagContainer.HasTagExact(Tag);
	}
	return TagContainer.HasTag(Tag);
}

bool UBlueprintGameplayTagLibrary::HasAnyTags(const FGameplayTagContainer& TagContainer, const FGameplayTagContainer& OtherContainer, bool bExactMatch)
{
	if (bExactMatch)
	{
		return TagContainer.HasAnyExact(OtherContainer);
	}
	return TagContainer.HasAny(OtherContainer);
}

bool UBlueprintGameplayTagLibrary::HasAllTags(const FGameplayTagContainer& TagContainer, const FGameplayTagContainer& OtherContainer, bool bExactMatch)
{
	if (bExactMatch)
	{
		return TagContainer.HasAllExact(OtherContainer);
	}
	return TagContainer.HasAll(OtherContainer);
}

FGameplayTagContainer UBlueprintGameplayTagLibrary::Filter(const FGameplayTagContainer& TagContainer, const FGameplayTagContainer& OtherContainer, bool bExactMatch)
{
	if (bExactMatch)
	{
		return TagContainer.FilterExact(OtherContainer);
	}
	return TagContainer.Filter(OtherContainer);
}

bool UBlueprintGameplayTagLibrary::IsTagQueryEmpty(const FGameplayTagQuery& TagQuery)
{
	return TagQuery.IsEmpty();
}

bool UBlueprintGameplayTagLibrary::DoesContainerMatchTagQuery(const FGameplayTagContainer& TagContainer, const FGameplayTagQuery& TagQuery)
{
	return TagQuery.Matches(TagContainer);
}

void UBlueprintGameplayTagLibrary::GetAllActorsOfClassMatchingTagQuery(UObject* WorldContextObject,
																	   TSubclassOf<AActor> ActorClass,
																	   const FGameplayTagQuery& GameplayTagQuery,
																	   TArray<AActor*>& OutActors)
{
	OutActors.Empty();

	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);

	// We do nothing if not class provided, rather than giving ALL actors!
	if (ActorClass && World)
	{
		bool bHasLoggedMissingInterface = false;
		for (TActorIterator<AActor> It(World, ActorClass); It; ++It)
		{
			AActor* Actor = *It;
			check(Actor != nullptr);
			if (IsValid(Actor))
			{
				IGameplayTagAssetInterface* GameplayTagAssetInterface = Cast<IGameplayTagAssetInterface>(Actor);
				if (GameplayTagAssetInterface != nullptr)
				{
					FGameplayTagContainer OwnedGameplayTags;
					GameplayTagAssetInterface->GetOwnedGameplayTags(OwnedGameplayTags);

					if (OwnedGameplayTags.MatchesQuery(GameplayTagQuery))
					{
						OutActors.Add(Actor);
					}
				}
				else
				{
					if (!bHasLoggedMissingInterface)
					{
						UE_LOGF(LogGameplayTags, Warning,
							"At least one actor (%ls) of class %ls does not implement IGameplTagAssetInterface.  Unable to find owned tags, so cannot determine if actor matches gameplay tag query.  Presuming it does not.",
							*Actor->GetName(), *ActorClass->GetName());
						bHasLoggedMissingInterface = true;
					}
				}
			}
		}
	}
}

bool UBlueprintGameplayTagLibrary::EqualEqual_GameplayTagContainer(const FGameplayTagContainer& A, const FGameplayTagContainer& B)
{
	return A == B;
}

bool UBlueprintGameplayTagLibrary::NotEqual_GameplayTagContainer(const FGameplayTagContainer& A, const FGameplayTagContainer& B)
{
	return A != B;
}

FGameplayTagContainer UBlueprintGameplayTagLibrary::MakeLiteralGameplayTagContainer(FGameplayTagContainer Value)
{
	return Value;
}

FGameplayTagContainer UBlueprintGameplayTagLibrary::MakeGameplayTagContainerFromArray(const TArray<FGameplayTag>& GameplayTags)
{
	return FGameplayTagContainer::CreateFromArray(GameplayTags);
}

FGameplayTagContainer UBlueprintGameplayTagLibrary::MakeGameplayTagContainerFromTag(FGameplayTag SingleTag)
{
	return FGameplayTagContainer(SingleTag);
}

void UBlueprintGameplayTagLibrary::BreakGameplayTagContainer(const FGameplayTagContainer& GameplayTagContainer, TArray<FGameplayTag>& GameplayTags)
{
	GameplayTagContainer.GetGameplayTagArray(GameplayTags);
}

FGameplayTagQuery UBlueprintGameplayTagLibrary::MakeGameplayTagQuery(FGameplayTagQuery TagQuery)
{
	return TagQuery;
}

FGameplayTagQuery UBlueprintGameplayTagLibrary::MakeGameplayTagQuery_MatchAnyTags(const FGameplayTagContainer& InTags)
{
	return FGameplayTagQuery::MakeQuery_MatchAnyTags(InTags);
}

FGameplayTagQuery UBlueprintGameplayTagLibrary::MakeGameplayTagQuery_MatchAllTags(const FGameplayTagContainer& InTags)
{
	return FGameplayTagQuery::MakeQuery_MatchAllTags(InTags);
}

FGameplayTagQuery UBlueprintGameplayTagLibrary::MakeGameplayTagQuery_MatchNoTags(const FGameplayTagContainer& InTags)
{
	return FGameplayTagQuery::MakeQuery_MatchNoTags(InTags);
}

bool UBlueprintGameplayTagLibrary::HasAllMatchingGameplayTags(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface, const FGameplayTagContainer& OtherContainer)
{
	if (IGameplayTagAssetInterface* Interface = TagContainerInterface.GetInterface())
	{
		return Interface->HasAllMatchingGameplayTags(OtherContainer);
	}
	return OtherContainer.IsEmpty();
}

bool UBlueprintGameplayTagLibrary::DoesTagAssetInterfaceHaveTag(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface, FGameplayTag Tag)
{
	if (IGameplayTagAssetInterface* Interface = TagContainerInterface.GetInterface())
	{
		return Interface->HasMatchingGameplayTag(Tag);
	}
	return false;
}

bool UBlueprintGameplayTagLibrary::HasAnyMatchingGameplayTags(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface, const FGameplayTagContainer& OtherContainer)
{
	if (IGameplayTagAssetInterface* Interface = TagContainerInterface.GetInterface())
	{
		return Interface->HasAnyMatchingGameplayTags(OtherContainer);
	}
	return false;
}

void UBlueprintGameplayTagLibrary::AppendGameplayTagContainers(FGameplayTagContainer& InOutTagContainer, const FGameplayTagContainer& InTagContainer)
{
	InOutTagContainer.AppendTags(InTagContainer);
}

void UBlueprintGameplayTagLibrary::AddGameplayTag(FGameplayTagContainer& TagContainer, FGameplayTag Tag)
{
	TagContainer.AddTag(Tag);
}

bool UBlueprintGameplayTagLibrary::RemoveGameplayTag(FGameplayTagContainer& TagContainer, FGameplayTag Tag)
{
	return TagContainer.RemoveTag(Tag);
}

bool UBlueprintGameplayTagLibrary::NotEqual_TagTag(FGameplayTag A, FString B)
{
	return A.ToString() != B;
}

bool UBlueprintGameplayTagLibrary::NotEqual_TagContainerTagContainer(FGameplayTagContainer A, FString B)
{
	FGameplayTagContainer TagContainer;

	const FString OpenParenthesesStr("(");
	const FString CloseParenthesesStr(")");

	// Convert string to Tag Container before compare
	FString TagString = MoveTemp(B);
	if (TagString.StartsWith(OpenParenthesesStr, ESearchCase::CaseSensitive) && TagString.EndsWith(CloseParenthesesStr, ESearchCase::CaseSensitive))
	{
		TagString.LeftChopInline(1, EAllowShrinking::No);
		TagString.RightChopInline(1, EAllowShrinking::No);

		const FString EqualStr("=");

		TagString.Split(EqualStr, nullptr, &TagString, ESearchCase::CaseSensitive);

		TagString.LeftChopInline(1, EAllowShrinking::No);
		TagString.RightChopInline(1, EAllowShrinking::No);

		FString ReadTag;
		FString Remainder;

		const FString CommaStr(",");
		const FString QuoteStr("\"");

		while (TagString.Split(CommaStr, &ReadTag, &Remainder, ESearchCase::CaseSensitive))
		{
			ReadTag.Split(EqualStr, nullptr, &ReadTag, ESearchCase::CaseSensitive);
			if (ReadTag.EndsWith(CloseParenthesesStr, ESearchCase::CaseSensitive))
			{
				ReadTag.LeftChopInline(1, EAllowShrinking::No);
				if (ReadTag.StartsWith(QuoteStr, ESearchCase::CaseSensitive) && ReadTag.EndsWith(QuoteStr, ESearchCase::CaseSensitive))
				{
					ReadTag.LeftChopInline(1, EAllowShrinking::No);
					ReadTag.RightChopInline(1, EAllowShrinking::No);
				}
			}
			TagString = Remainder;

			const FGameplayTag Tag = FGameplayTag::RequestGameplayTag(FName(*ReadTag));
			TagContainer.AddTag(Tag);
		}
		if (Remainder.IsEmpty())
		{
			Remainder = MoveTemp(TagString);
		}
		if (!Remainder.IsEmpty())
		{
			Remainder.Split(EqualStr, nullptr, &Remainder, ESearchCase::CaseSensitive);
			if (Remainder.EndsWith(CloseParenthesesStr, ESearchCase::CaseSensitive))
			{
				Remainder.LeftChopInline(1, EAllowShrinking::No);
				if (Remainder.StartsWith(QuoteStr, ESearchCase::CaseSensitive) && Remainder.EndsWith(QuoteStr, ESearchCase::CaseSensitive))
				{
					Remainder.LeftChopInline(1, EAllowShrinking::No);
					Remainder.RightChopInline(1, EAllowShrinking::No);
				}
			}
			const FGameplayTag Tag = FGameplayTag::RequestGameplayTag(FName(*Remainder));
			TagContainer.AddTag(Tag);
		}
	}

	return A != TagContainer;
}

FGameplayTagContainer UBlueprintGameplayTagLibrary::GetOwnedGameplayTags(TScriptInterface<IGameplayTagAssetInterface> TagContainerInterface)
{
	// Note:
	// Why do this instead of just letting the BP Compiler use the InOut parameter of the original function?
	//   Because the BP Compiler will store the out values in the event graph as a hidden property, which will
	//   cause issues because the tags will never get reset, and will always be appended to, most all the implementations
	//   of the GetOwnedGameplayTags allow you to continually Append tags to the container, but because the fact that the
	//   inout is hidden in BP, you don't get the understand or option really to clear the tags, so this is a safer way.

	FGameplayTagContainer OwnedTags;
	if (TagContainerInterface)
	{
		TagContainerInterface->GetOwnedGameplayTags(OwnedTags);
	}

	return OwnedTags;
}

TScriptInterface<IGameplayTagAssetInterface> UBlueprintGameplayTagLibrary::Conv_ObjectToGameplayTagAssetInterface(UObject* InObject)
{
	return TScriptInterface<IGameplayTagAssetInterface>(InObject);
}

FString UBlueprintGameplayTagLibrary::GetDebugStringFromGameplayTagContainer(const FGameplayTagContainer& TagContainer)
{
	return TagContainer.ToStringSimple();
}

FString UBlueprintGameplayTagLibrary::GetDebugStringFromGameplayTag(FGameplayTag GameplayTag)
{
	return GameplayTag.ToString();
}

