// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagAssetInterface.h"
#include "BlueprintGameplayTagLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagAssetInterface)

UGameplayTagAssetInterface::UGameplayTagAssetInterface(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

bool IGameplayTagAssetInterface::HasMatchingGameplayTag(FGameplayTag TagToCheck) const
{
	FGameplayTagContainer OwnedTags;
	GetOwnedGameplayTags(OwnedTags);

	return OwnedTags.HasTag(TagToCheck);
}

bool IGameplayTagAssetInterface::HasAllMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
{
	FGameplayTagContainer OwnedTags;
	GetOwnedGameplayTags(OwnedTags);

	return OwnedTags.HasAll(TagContainer);
}

bool IGameplayTagAssetInterface::HasAnyMatchingGameplayTags(const FGameplayTagContainer& TagContainer) const
{
	FGameplayTagContainer OwnedTags;
	GetOwnedGameplayTags(OwnedTags);

	return OwnedTags.HasAny(TagContainer);
}

FGameplayTagContainer IGameplayTagAssetInterface::BP_GetOwnedGameplayTags() const
{
	// This is just a hack to forward to the proper non-member function.  The idea being in the future, 
	// we can redirect this member function to the non-member function in the future once we allow the CoreRedirect
	// of the 'self' pin.
	TScriptInterface<IGameplayTagAssetInterface> GameplayTagAssetInterface;
	IGameplayTagAssetInterface* NonConstThis = const_cast<IGameplayTagAssetInterface*>(this);
	GameplayTagAssetInterface.SetObject(Cast<UObject>(NonConstThis));
	GameplayTagAssetInterface.SetInterface(NonConstThis);

	return UBlueprintGameplayTagLibrary::GetOwnedGameplayTags(GameplayTagAssetInterface);
}
