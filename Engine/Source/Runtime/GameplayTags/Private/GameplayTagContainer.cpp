// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagContainer.h"
#include "HAL/IConsoleManager.h"
#include "UObject/CoreNet.h"
#include "UObject/ObjectSaveContext.h"
#include "UObject/UnrealType.h"
#include "Engine/PackageMapClient.h"
#include "UObject/Package.h"
#include "Engine/NetConnection.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "Misc/OutputDeviceNull.h"
#include "JsonObjectConverter.h"

// For dynamic serialization support
#include "Net/Core/NetToken/NetTokenExportContext.h"
#include "GameplayTagTokenStore.h"
#include <Net/Core/Trace/NetTrace.h>

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagContainer)

const FGameplayTag FGameplayTag::EmptyTag;
const FGameplayTagContainer FGameplayTagContainer::EmptyContainer;
const FGameplayTagQuery FGameplayTagQuery::EmptyQuery;

DEFINE_STAT(STAT_FGameplayTagContainer_DoesTagContainerMatch);

DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::RemoveTagByExplicitName"), STAT_FGameplayTagContainer_RemoveTagByExplicitName, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::FillParentTags"), STAT_FGameplayTagContainer_FillParentTags, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::GetGameplayTagParents"), STAT_FGameplayTagContainer_GetGameplayTagParents, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::Filter"), STAT_FGameplayTagContainer_Filter, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::AppendTags"), STAT_FGameplayTagContainer_AppendTags, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::AppendMatchingTags"), STAT_FGameplayTagContainer_AppendMatchingTags, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::AddTag"), STAT_FGameplayTagContainer_AddTag, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::RemoveTag"), STAT_FGameplayTagContainer_RemoveTag, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTagContainer::RemoveTags"), STAT_FGameplayTagContainer_RemoveTags, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTag::GetSingleTagContainer"), STAT_FGameplayTag_GetSingleTagContainer, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTag::MatchesTag"), STAT_FGameplayTag_MatchesTag, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTag::MatchesAny"), STAT_FGameplayTag_MatchesAny, STATGROUP_GameplayTags);
DECLARE_CYCLE_STAT(TEXT("FGameplayTag::NetSerialize"), STAT_FGameplayTag_NetSerialize, STATGROUP_GameplayTags);

static bool GEnableGameplayTagDetailedStats = false;
static FAutoConsoleVariableRef CVarGameplayTagDetailedStats(TEXT("GameplayTags.EnableDetailedStats"), GEnableGameplayTagDetailedStats, TEXT("Runtime toggle for verbose CPU profiling stats"), ECVF_Default);

static bool GOldReplaysUseGameplayTagFastReplication = true;
static FAutoConsoleVariableRef CVarOldReplaysUseGameplayTagFastReplication(TEXT("GameplayTags.OldReplaysUseFastReplication"), GOldReplaysUseGameplayTagFastReplication, TEXT("When loading an outdated replay (before dynamic replication), do we assume it used fast replication?"), ECVF_Default);

/**
 *	Replicates a tag in a packed format:
 *	-A segment of NetIndexFirstBitSegment bits are always replicated.
 *	-Another bit is replicated to indicate "more"
 *	-If "more", then another segment of (MaxBits - NetIndexFirstBitSegment) length is replicated.
 *	
 *	This format is basically the same as SerializeIntPacked, except that there are only 2 segments and they are not the same size.
 *	The gameplay tag system is able to exploit knoweledge in what tags are frequently replicated to ensure they appear in the first segment.
 *	Making frequently replicated tags as cheap as possible. 
 *	
 *	
 *	Setting up your project to take advantage of the packed format.
 *	-Run a normal networked game on non shipping build. 
 *	-After some time, run console command "GameplayTags.PrintReport" or set "GameplayTags.PrintReportOnShutdown 1" cvar.
 *	-This will generate information on the server log about what tags replicate most frequently.
 *	-Take this list and put it in DefaultGameplayTags.ini.
 *	-CommonlyReplicatedTags is the ordered list of tags.
 *	-NetIndexFirstBitSegment is the number of bits (not including the "more" bit) for the first segment.
 *
 */
void SerializeTagNetIndexPacked(FArchive& Ar, FGameplayTagNetIndex& Value, const int32 NetIndexFirstBitSegment, const int32 MaxBits)
{
	// Case where we have no segment or the segment is larger than max bits
	if (NetIndexFirstBitSegment <= 0 || NetIndexFirstBitSegment >= MaxBits)
	{
		if (Ar.IsLoading())
		{
			Value = 0;
		}
		Ar.SerializeBits(&Value, MaxBits);
		return;
	}


	const uint32 BitMasks[] = {0x00, 0x01, 0x03, 0x07, 0x0f, 0x1f, 0x3f, 0x7f, 0xff, 0x1ff, 0x3ff, 0x7ff, 0xfff, 0x1fff, 0x3fff, 0x7fff, 0xffff};
	const uint32 MoreBits[] = {0x00, 0x01, 0x02, 0x04, 0x08, 0x10, 0x20, 0x40, 0x80, 0x100, 0x200, 0x400, 0x800, 0x1000, 0x2000, 0x4000, 0x8000};

	const int32 FirstSegment = NetIndexFirstBitSegment;
	const int32 SecondSegment = MaxBits - NetIndexFirstBitSegment;

	if (Ar.IsSaving())
	{
		uint32 Mask = BitMasks[FirstSegment];
		if (Value > Mask)
		{
			uint32 FirstDataSegment = ((Value & Mask) | MoreBits[FirstSegment+1]);
			uint32 SecondDataSegment = (Value >> FirstSegment);

			uint32 SerializedValue = FirstDataSegment | (SecondDataSegment << (FirstSegment+1));				

			Ar.SerializeBits(&SerializedValue, MaxBits + 1);
		}
		else
		{
			uint32 SerializedValue = Value;
			Ar.SerializeBits(&SerializedValue, NetIndexFirstBitSegment + 1);
		}

	}
	else
	{
		uint32 FirstData = 0;
		Ar.SerializeBits(&FirstData, FirstSegment + 1);
		uint32 More = FirstData & MoreBits[FirstSegment+1];
		if (More)
		{
			uint32 SecondData = 0;
			Ar.SerializeBits(&SecondData, SecondSegment);
			Value = IntCastChecked<uint16, uint32>(SecondData << FirstSegment);
			Value |= (FirstData & BitMasks[FirstSegment]);
		}
		else
		{
			Value = IntCastChecked<uint16, uint32>(FirstData);
		}

	}
}


/** Helper class to parse/eval query token streams. */
class FQueryEvaluator
{
public:
	FQueryEvaluator(FGameplayTagQuery const& Q)
		: Query(Q), 
		CurStreamIdx(0), 
		Version(EGameplayTagQueryStreamVersion::LatestVersion), 
		bReadError(false)
	{}

	/** Evaluates the query against the given tag container and returns the result (true if matching, false otherwise). */
	bool Eval(FGameplayTagContainer const& Tags);

	/** Parses the token stream into an FExpr. */
	void Read(struct FGameplayTagQueryExpression& E);

private:
	FGameplayTagQuery const& Query;
	int32 CurStreamIdx;
	int32 Version;
	bool bReadError;

	bool EvalAnyTagsMatch(FGameplayTagContainer const& Tags, bool bSkip);
	bool EvalAllTagsMatch(FGameplayTagContainer const& Tags, bool bSkip);
	bool EvalNoTagsMatch(FGameplayTagContainer const& Tags, bool bSkip);
	bool EvalAnyTagsExactMatch(FGameplayTagContainer const& Tags, bool bSkip);
	bool EvalAllTagsExactMatch(FGameplayTagContainer const& Tags, bool bSkip);
	

	bool EvalAnyExprMatch(FGameplayTagContainer const& Tags, bool bSkip);
	bool EvalAllExprMatch(FGameplayTagContainer const& Tags, bool bSkip);
	bool EvalNoExprMatch(FGameplayTagContainer const& Tags, bool bSkip);

	bool EvalExpr(FGameplayTagContainer const& Tags, bool bSkip = false);
	void ReadExpr(struct FGameplayTagQueryExpression& E);

#if WITH_EDITOR
public:
	UEditableGameplayTagQuery* CreateEditableQuery();

private:
	UEditableGameplayTagQueryExpression* ReadEditableQueryExpr(UObject* ExprOuter);
	void ReadEditableQueryTags(UEditableGameplayTagQueryExpression* EditableQueryExpr);
	void ReadEditableQueryExprList(UEditableGameplayTagQueryExpression* EditableQueryExpr);
#endif // WITH_EDITOR

	/** Returns the next token in the stream. If there's a read error, sets bReadError and returns zero, so be sure to check that. */
	uint8 GetToken()
	{
		if (Query.QueryTokenStream.IsValidIndex(CurStreamIdx))
		{
			return Query.QueryTokenStream[CurStreamIdx++];
		}
		
		UE_LOGF(LogGameplayTags, Warning, "Error parsing FGameplayTagQuery!");
		bReadError = true; 
		return 0;
	}
};

bool FQueryEvaluator::Eval(FGameplayTagContainer const& Tags)
{
	CurStreamIdx = 0;

	// start parsing the set
	Version = GetToken();
	if (bReadError)
	{
		return false;
	}
	
	bool bRet = false;

	uint8 const bHasRootExpression = GetToken();
	if (!bReadError && bHasRootExpression)
	{
		bRet = EvalExpr(Tags);

	}

	ensure(CurStreamIdx == Query.QueryTokenStream.Num());
	return bRet;
}

void FQueryEvaluator::Read(FGameplayTagQueryExpression& E)
{
	E = FGameplayTagQueryExpression();
	CurStreamIdx = 0;

	if (Query.QueryTokenStream.Num() > 0)
	{
		// start parsing the set
		Version = GetToken();
		if (!bReadError)
		{
			uint8 const bHasRootExpression = GetToken();
			if (!bReadError && bHasRootExpression)
			{
				ReadExpr(E);
			}
		}

		ensure(CurStreamIdx == Query.QueryTokenStream.Num());
	}
}

void FQueryEvaluator::ReadExpr(FGameplayTagQueryExpression& E)
{
	E.ExprType = (EGameplayTagQueryExprType) GetToken();
	if (bReadError)
	{
		return;
	}
	
	if (E.UsesTagSet())
	{
		// parse tag set
		int32 NumTags = GetToken();
		if (bReadError)
		{
			return;
		}

		for (int32 Idx = 0; Idx < NumTags; ++Idx)
		{
			int32 const TagIdx = GetToken();
			if (bReadError)
			{
				return;
			}

			FGameplayTag Tag = Query.GetTagFromIndex(TagIdx);
			E.AddTag(Tag);
		}
	}
	else
	{
		// parse expr set
		int32 NumExprs = GetToken();
		if (bReadError)
		{
			return;
		}

		for (int32 Idx = 0; Idx < NumExprs; ++Idx)
		{
			FGameplayTagQueryExpression Exp;
			ReadExpr(Exp);
			E.AddExpr(Exp);
		}
	}
}


bool FQueryEvaluator::EvalAnyTagsMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;
	bool Result = false;

	// parse tagset
	int32 const NumTags = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumTags; ++Idx)
	{
		int32 const TagIdx = GetToken();
		if (bReadError)
		{
			return false;
		}

		if (bShortCircuit == false)
		{
			FGameplayTag Tag = Query.GetTagFromIndex(TagIdx);

			bool bHasTag = Tags.HasTag(Tag);

			if (bHasTag)
			{
				// one match is sufficient for a true result!
				bShortCircuit = true;
				Result = true;
			}
		}
	}

	return Result;
}

bool FQueryEvaluator::EvalAllTagsMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;

	// assume true until proven otherwise
	bool Result = true;

	// parse tagset
	int32 const NumTags = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumTags; ++Idx)
	{
		int32 const TagIdx = GetToken();
		if (bReadError)
		{
			return false;
		}

		if (bShortCircuit == false)
		{
			FGameplayTag const Tag = Query.GetTagFromIndex(TagIdx);
			bool const bHasTag = Tags.HasTag(Tag);

			if (bHasTag == false)
			{
				// one failed match is sufficient for a false result
				bShortCircuit = true;
				Result = false;
			}
		}
	}

	return Result;
}

bool FQueryEvaluator::EvalNoTagsMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;

	// assume true until proven otherwise
	bool Result = true;

	// parse tagset
	int32 const NumTags = GetToken();
	if (bReadError)
	{
		return false;
	}
	
	for (int32 Idx = 0; Idx < NumTags; ++Idx)
	{
		int32 const TagIdx = GetToken();
		if (bReadError)
		{
			return false;
		}

		if (bShortCircuit == false)
		{
			FGameplayTag const Tag = Query.GetTagFromIndex(TagIdx);
			bool const bHasTag = Tags.HasTag(Tag);

			if (bHasTag == true)
			{
				// one match is sufficient for a false result
				bShortCircuit = true;
				Result = false;
			}
		}
	}

	return Result;
}

bool FQueryEvaluator::EvalAnyTagsExactMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;
	bool Result = false;

	// parse tagset
	int32 const NumTags = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumTags; ++Idx)
	{
		int32 const TagIdx = GetToken();
		if (bReadError)
		{
			return false;
		}

		if (bShortCircuit == false)
		{
			FGameplayTag Tag = Query.GetTagFromIndex(TagIdx);

			bool bHasTag = Tags.HasTagExact(Tag);

			if (bHasTag)
			{
				// one match is sufficient for a true result!
				bShortCircuit = true;
				Result = true;
			}
		}
	}

	return Result;
}

bool FQueryEvaluator::EvalAllTagsExactMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;

	// assume true until proven otherwise
	bool Result = true;

	// parse tagset
	int32 const NumTags = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumTags; ++Idx)
	{
		int32 const TagIdx = GetToken();
		if (bReadError)
		{
			return false;
		}

		if (bShortCircuit == false)
		{
			FGameplayTag const Tag = Query.GetTagFromIndex(TagIdx);
			bool const bHasTag = Tags.HasTagExact(Tag);

			if (bHasTag == false)
			{
				// one failed match is sufficient for a false result
				bShortCircuit = true;
				Result = false;
			}
		}
	}

	return Result;
}

bool FQueryEvaluator::EvalAnyExprMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;

	// assume false until proven otherwise
	bool Result = false;

	// parse exprset
	int32 const NumExprs = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumExprs; ++Idx)
	{
		bool const bExprResult = EvalExpr(Tags, bShortCircuit);
		if (bShortCircuit == false)
		{
			if (bExprResult == true)
			{
				// one match is sufficient for true result
				Result = true;
				bShortCircuit = true;
			}
		}
	}

	return Result;
}
bool FQueryEvaluator::EvalAllExprMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;

	// assume true until proven otherwise
	bool Result = true;

	// parse exprset
	int32 const NumExprs = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumExprs; ++Idx)
	{
		bool const bExprResult = EvalExpr(Tags, bShortCircuit);
		if (bShortCircuit == false)
		{
			if (bExprResult == false)
			{
				// one fail is sufficient for false result
				Result = false;
				bShortCircuit = true;
			}
		}
	}

	return Result;
}
bool FQueryEvaluator::EvalNoExprMatch(FGameplayTagContainer const& Tags, bool bSkip)
{
	bool bShortCircuit = bSkip;

	// assume true until proven otherwise
	bool Result = true;

	// parse exprset
	int32 const NumExprs = GetToken();
	if (bReadError)
	{
		return false;
	}

	for (int32 Idx = 0; Idx < NumExprs; ++Idx)
	{
		bool const bExprResult = EvalExpr(Tags, bShortCircuit);
		if (bShortCircuit == false)
		{
			if (bExprResult == true)
			{
				// one match is sufficient for fail result
				Result = false;
				bShortCircuit = true;
			}
		}
	}

	return Result;
}


bool FQueryEvaluator::EvalExpr(FGameplayTagContainer const& Tags, bool bSkip)
{
	EGameplayTagQueryExprType const ExprType = (EGameplayTagQueryExprType) GetToken();
	if (bReadError)
	{
		return false;
	}

	// emit exprdata
	switch (ExprType)
	{
	case EGameplayTagQueryExprType::AnyTagsMatch:
		return EvalAnyTagsMatch(Tags, bSkip);
	case EGameplayTagQueryExprType::AllTagsMatch:
		return EvalAllTagsMatch(Tags, bSkip);
	case EGameplayTagQueryExprType::NoTagsMatch:
		return EvalNoTagsMatch(Tags, bSkip);
	case EGameplayTagQueryExprType::AnyTagsExactMatch:
		return EvalAnyTagsExactMatch(Tags, bSkip);
	case EGameplayTagQueryExprType::AllTagsExactMatch:
		return EvalAllTagsExactMatch(Tags, bSkip);

	case EGameplayTagQueryExprType::AnyExprMatch:
		return EvalAnyExprMatch(Tags, bSkip);
	case EGameplayTagQueryExprType::AllExprMatch:
		return EvalAllExprMatch(Tags, bSkip);
	case EGameplayTagQueryExprType::NoExprMatch:
		return EvalNoExprMatch(Tags, bSkip);
	}

	ensureAlwaysMsgf(false, TEXT("Encountered invalid query expression type: %s"), *UEnum::GetValueAsString(ExprType));
	return false;
}

const TArray<FGameplayTag>& FGameplayTagContainer::GetGameplayTagArray() const
{
	return GameplayTags;
}

FGameplayTagContainer& FGameplayTagContainer::operator=(FGameplayTagContainer const& Other)
{
	// Guard against self-assignment
	if (this == &Other)
	{
		return *this;
	}
	GameplayTags.Empty(Other.GameplayTags.Num());
	GameplayTags.Append(Other.GameplayTags);

	ParentTags.Empty(Other.ParentTags.Num());
	ParentTags.Append(Other.ParentTags);

	return *this;
}

FGameplayTagContainer& FGameplayTagContainer::operator=(FGameplayTagContainer&& Other)
{
	GameplayTags = MoveTemp(Other.GameplayTags);
	ParentTags = MoveTemp(Other.ParentTags);
	return *this;
}

bool FGameplayTagContainer::operator==(FGameplayTagContainer const& Other) const
{
	// This is to handle the case where the two containers are in different orders
	if (GameplayTags.Num() != Other.GameplayTags.Num())
	{
		return false;
	}

	return HasAllExact(Other);
}

bool FGameplayTagContainer::operator!=(FGameplayTagContainer const& Other) const
{
	return !operator==(Other);
}

bool FGameplayTagContainer::RemoveTagByExplicitName(const FName& TagName)
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_RemoveTagByExplicitName);

	// TODO NDarnell Why are we doing this instead of just return RemoveTag(FGameplayTag(TagName));
	for (auto GameplayTag : this->GameplayTags)
	{
		if (GameplayTag.GetTagName() == TagName)
		{
			RemoveTag(GameplayTag);
			return true;
		}
	}

	return false;
}

void FGameplayTagContainer::AddParentsForTag(const FGameplayTag& Tag)
{
	UGameplayTagsManager::Get().ExtractParentTags(Tag, ParentTags);
}

void FGameplayTagContainer::FillParentTags()
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_FillParentTags);

	ParentTags.Reset();

	if (GameplayTags.Num() > 0)
	{
		UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
		for (const FGameplayTag& Tag : GameplayTags)
		{
			TagManager.ExtractParentTags(Tag, ParentTags);
		}
	}
}

FGameplayTagContainer FGameplayTagContainer::GetGameplayTagParents() const
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_GetGameplayTagParents);

	FGameplayTagContainer ResultContainer;
	ResultContainer.GameplayTags = GameplayTags;

	// Add parent tags to explicit tags, the rest got copied over already
	for (const FGameplayTag& Tag : ParentTags)
	{
		ResultContainer.GameplayTags.AddUnique(Tag);
	}

	return ResultContainer;
}

FGameplayTagContainer FGameplayTagContainer::Filter(const FGameplayTagContainer& OtherContainer) const
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_Filter);

	FGameplayTagContainer ResultContainer;

	for (const FGameplayTag& Tag : GameplayTags)
	{
		if (Tag.MatchesAny(OtherContainer))
		{
			ResultContainer.AddTagFast(Tag);
		}
	}

	return ResultContainer;
}

FGameplayTagContainer FGameplayTagContainer::FilterExact(const FGameplayTagContainer& OtherContainer) const
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_Filter);

	FGameplayTagContainer ResultContainer;

	for (const FGameplayTag& Tag : GameplayTags)
	{
		if (Tag.MatchesAnyExact(OtherContainer))
		{
			ResultContainer.AddTagFast(Tag);
		}
	}

	return ResultContainer;
}

bool FGameplayTagContainer::MatchesQuery(const FGameplayTagQuery& Query) const
{
	return Query.Matches(*this);
}

void FGameplayTagContainer::AppendTags(FGameplayTagContainer const& Other)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_AppendTags, GEnableGameplayTagDetailedStats);
	if (Other.IsEmpty())
	{
		return;
	}

	int32 OldTagNum = GameplayTags.Num();
	GameplayTags.Reserve(OldTagNum + Other.GameplayTags.Num());
	// Add other container's tags to our own
	for(const FGameplayTag& OtherTag : Other.GameplayTags)
	{
		int32 SearchIndex = 0;
		while (true)
		{
			if (SearchIndex >= OldTagNum)
			{
				// Stop searching once we've looked at all existing tags, this is faster when appending large containers
				GameplayTags.Add(OtherTag);
				break;
			}
			else if (GameplayTags[SearchIndex] == OtherTag)
			{
				// Matching tag found, stop searching
				break;
			}

			SearchIndex++;
		}
	}

	// This function is called enough that the code duplication is faster than a lambda
	OldTagNum = ParentTags.Num();
	ParentTags.Reserve(OldTagNum + Other.ParentTags.Num());
	for (const FGameplayTag& OtherTag : Other.ParentTags)
	{
		int32 SearchIndex = 0;
		while (true)
		{
			if (SearchIndex >= OldTagNum)
			{
				ParentTags.Add(OtherTag);
				break;
			}
			else if (ParentTags[SearchIndex] == OtherTag)
			{
				break;
			}

			SearchIndex++;
		}
	}
}

void FGameplayTagContainer::AppendMatchingTags(FGameplayTagContainer const& OtherA, FGameplayTagContainer const& OtherB)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_AppendMatchingTags, GEnableGameplayTagDetailedStats);

	for(const FGameplayTag& OtherATag : OtherA.GameplayTags)
	{
		if (OtherATag.MatchesAny(OtherB))
		{
			AddTag(OtherATag);
		}
	}
}

void FGameplayTagContainer::AddTag(const FGameplayTag& TagToAdd)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_AddTag, GEnableGameplayTagDetailedStats);

	if (TagToAdd.IsValid())
	{
		// Don't want duplicate tags
		GameplayTags.AddUnique(TagToAdd);

		UGameplayTagsManager::Get().ExtractParentTags(TagToAdd, ParentTags);
	}
}

void FGameplayTagContainer::AddTagFast(const FGameplayTag& TagToAdd)
{
	GameplayTags.Add(TagToAdd);
	UGameplayTagsManager::Get().ExtractParentTags(TagToAdd, ParentTags);
}

bool FGameplayTagContainer::AddLeafTag(const FGameplayTag& TagToAdd)
{
	// Check tag is not already explicitly in container
	if (HasTagExact(TagToAdd))
	{
		return true;
	}

	// If this tag is parent of explicitly added tag, fail
	if (HasTag(TagToAdd))
	{
		return false;
	}

	TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(TagToAdd);

	if (ensureMsgf(TagNode.IsValid(), TEXT("AddLeafTag passed invalid gameplay tag %s, only registered tags can be queried"), *TagToAdd.GetTagName().ToString()))
	{
		// Remove any tags in the container that are a parent to TagToAdd
		for (const FGameplayTag& ParentTag : TagNode->GetSingleTagContainer().ParentTags)
		{
			if (HasTagExact(ParentTag))
			{
				RemoveTag(ParentTag);
			}
		}
	}

	// Add the tag
	AddTag(TagToAdd);
	return true;
}

bool FGameplayTagContainer::RemoveTag(const FGameplayTag& TagToRemove, bool bDeferParentTags)
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_RemoveTag);

	int32 NumChanged = GameplayTags.RemoveSingle(TagToRemove);

	if (NumChanged > 0)
	{
		if (!bDeferParentTags)
		{
			// Have to recompute parent table from scratch because there could be duplicates providing the same parent tag
			FillParentTags();
		}
		return true;
	}
	return false;
}

void FGameplayTagContainer::RemoveTags(const FGameplayTagContainer& TagsToRemove)
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTagContainer_RemoveTags);

	int32 NumChanged = 0;

	for (auto Tag : TagsToRemove)
	{
		NumChanged += GameplayTags.RemoveSingle(Tag);
	}

	if (NumChanged > 0)
	{
		// Recompute once at the end
		FillParentTags();
	}
}

void FGameplayTagContainer::Reset(int32 Slack)
{
	GameplayTags.Reset(Slack);

	// ParentTags is usually around size of GameplayTags on average
	ParentTags.Reset(Slack);
}

bool FGameplayTagContainer::Serialize(FStructuredArchive::FSlot Slot)
{
	FArchive& UnderlyingArchive = Slot.GetUnderlyingArchive();

	const bool bOldTagVer = UnderlyingArchive.UEVer() < VER_UE4_GAMEPLAY_TAG_CONTAINER_TAG_TYPE_CHANGE;
	
	if (bOldTagVer)
	{
		TArray<FName> Tags_DEPRECATED;
		Slot << Tags_DEPRECATED;
		// Too old to deal with
		UE_LOGF(LogGameplayTags, Error, "Failed to load old GameplayTag container, too old to migrate correctly");
	}
	else
	{
		Slot << GameplayTags;
	}
	
	// Only do redirects for real loads, not for duplicates or recompiles
	if (UnderlyingArchive.IsLoading() )
	{
		if (UnderlyingArchive.IsPersistent() && !(UnderlyingArchive.GetPortFlags() & PPF_Duplicate) && !(UnderlyingArchive.GetPortFlags() & PPF_DuplicateForPIE))
		{
			// Rename any tags that may have changed by the ini file.  Redirects can happen regardless of version.
			// Regardless of version, want loading to have a chance to handle redirects
			UGameplayTagsManager::Get().GameplayTagContainerLoaded(*this, UnderlyingArchive.GetSerializedProperty());
		}

		FillParentTags();
	}

	if (UnderlyingArchive.IsSaving())
	{

#if WITH_EDITOR
		FObjectSavePackageSerializeContext* SaveContext = nullptr;
		bool bCreateDependencies = false;
		if (UnderlyingArchive.IsCooking())
		{
			FArchiveSavePackageData* SavePackageData = const_cast<FArchive&>(UnderlyingArchive).GetSavePackageData();
			check(SavePackageData);
			SaveContext = &SavePackageData->SavePackageContext;

			if (SaveContext->IsHarvestingCookDependencies())
			{
				bCreateDependencies = true;
			}
		}
#endif //#if WITH_EDITOR

		// This marks the saved name for later searching
		for (const FGameplayTag& Tag : GameplayTags)
		{
			UnderlyingArchive.MarkSearchableName(FGameplayTag::StaticStruct(), Tag.TagName);

#if WITH_EDITOR
			if (bCreateDependencies)
			{
				SaveContext->AddCookLoadDependency(UGameplayTagsManager::Get().CreateCookDependency(Tag));
			}
#endif //#if WITH_EDITOR

		}
	}

	return true;
}

FString FGameplayTagContainer::ToString() const
{
	FString ExportString;
	ExportString.Reserve(2048);
	FGameplayTagContainer::StaticStruct()->ExportText(ExportString, this, this, nullptr, 0, nullptr);

	return ExportString;
}

void FGameplayTagContainer::FromExportString(const FString& ExportString, int32 PortFlags)
{
	Reset();

	FOutputDeviceNull NullOut;
	FGameplayTagContainer::StaticStruct()->ImportText(*ExportString, this, nullptr, PortFlags, &NullOut, TEXT("FGameplayTagContainer"), true);
}

bool FGameplayTagContainer::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	// Call default import, but skip the native callback to avoid recursion
	Buffer = FGameplayTagContainer::StaticStruct()->ImportText(Buffer, this, Parent, PortFlags, ErrorText, TEXT("FGameplayTagContainer"), false);

	if (Buffer)
	{
		// Clear out any invalid tags that got stripped
		GameplayTags.Remove(FGameplayTag());

		// Compute parent tags
		FillParentTags();	
	}
	return true;
}

void FGameplayTagContainer::PostScriptConstruct()
{
	FillParentTags();
}

FString FGameplayTagContainer::ToStringSimple(bool bQuoted) const
{
	FString RetString;
	for (int i = 0; i < GameplayTags.Num(); ++i)
	{
		if (bQuoted)
		{
			RetString += TEXT("\"");
		}
		RetString += GameplayTags[i].ToString();
		if (bQuoted)
		{
			RetString += TEXT("\"");
		}
		
		if (i < GameplayTags.Num() - 1)
		{
			RetString += TEXT(", ");
		}
	}
	return RetString;
}

TArray<FString> FGameplayTagContainer::ToStringsMaxLen(int32 MaxLen) const
{
	// caveat, if MaxLen < than a tag string, full string will be put in array (as a single line in the array)
	// since this is used for debug output.  If need to clamp, it can be added.  Also, strings will end in ", " to 
	// avoid extra complication.
	TArray<FString> RetStrings;
	FString CurLine;
	CurLine.Reserve(MaxLen);
	for (int32 i = 0; i < GameplayTags.Num(); ++i)
	{
		FString TagString = GameplayTags[i].ToString();
		if (i < GameplayTags.Num() - 1)
		{
			TagString += TEXT(",");
		}
		// Add 1 for space
		if (CurLine.Len() + TagString.Len() + 1 >= MaxLen)
		{
			RetStrings.Add(CurLine);
			CurLine = TagString;
		} 
		else
		{
			CurLine += TagString + TEXT(" ");
		}
	}
	if (CurLine.Len() > 0)
	{
		RetStrings.Add(CurLine);
	}
	return RetStrings;
}

bool FGameplayTagContainer::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	// 1st bit to indicate empty tag container or not (empty tag containers are frequently replicated). Early out if empty.
	uint8 IsEmpty = GameplayTags.Num() == 0;
	Ar.SerializeBits(&IsEmpty, 1);
	if (IsEmpty)
	{
		if (GameplayTags.Num() > 0)
		{
			Reset();
		}
		bOutSuccess = true;
		return true;
	}

	// -------------------------------------------------------

	const int32 NumBitsForContainerSize = UGameplayTagsManager::Get().NumBitsForContainerSize;

	if (Ar.IsSaving())
	{
		uint16 NumTags = IntCastChecked<uint16, int32>(GameplayTags.Num());
		const uint16 MaxSize = (1 << NumBitsForContainerSize) - 1;
		if (!ensureMsgf(GameplayTags.Num() <= MaxSize, TEXT("TagContainer has %d elements when max is %d! Tags: %s"), GameplayTags.Num(), MaxSize, *ToStringSimple()))
		{
			NumTags = MaxSize;
		}
		
		Ar.SerializeBits(&NumTags, NumBitsForContainerSize);
		for (int32 idx=0; idx < NumTags;++idx)
		{
			FGameplayTag& Tag = GameplayTags[idx];
			Tag.NetSerialize_Packed(Ar, Map, bOutSuccess);

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			UGameplayTagsManager::Get().NotifyTagReplicated(Tag, true);
#endif
		}
	}
	else
	{
		// No Common Container tags, just replicate this like normal
		uint16 NumTags = 0;
		Ar.SerializeBits(&NumTags, NumBitsForContainerSize);

		GameplayTags.Empty(NumTags);
		GameplayTags.AddDefaulted(NumTags);
		for (int32 idx = 0; idx < NumTags; ++idx)
		{
			GameplayTags[idx].NetSerialize_Packed(Ar, Map, bOutSuccess);
		}
		FillParentTags();
	}


	bOutSuccess  = true;
	return true;
}

FText FGameplayTagContainer::ToMatchingText(EGameplayContainerMatchType MatchType, bool bInvertCondition) const
{
	enum class EMatchingTypes : int8
	{
		Inverted	= 0x01,
		All			= 0x02
	};

#define LOCTEXT_NAMESPACE "FGameplayTagContainer"
	const FText MatchingDescription[] =
	{
		LOCTEXT("MatchesAnyGameplayTags", "Has any tags in set: {GameplayTagSet}"),
		LOCTEXT("NotMatchesAnyGameplayTags", "Does not have any tags in set: {GameplayTagSet}"),
		LOCTEXT("MatchesAllGameplayTags", "Has all tags in set: {GameplayTagSet}"),
		LOCTEXT("NotMatchesAllGameplayTags", "Does not have all tags in set: {GameplayTagSet}")
	};
#undef LOCTEXT_NAMESPACE

	int32 DescriptionIndex = bInvertCondition ? static_cast<int32>(EMatchingTypes::Inverted) : 0;
	switch (MatchType)
	{
		case EGameplayContainerMatchType::All:
			DescriptionIndex |= static_cast<int32>(EMatchingTypes::All);
			break;

		case EGameplayContainerMatchType::Any:
			break;

		default:
			UE_LOGF(LogGameplayTags, Warning, "Invalid value for TagsToMatch (EGameplayContainerMatchType) %d.  Should only be Any or All.", static_cast<int32>(MatchType));
			break;
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("GameplayTagSet"), FText::FromString(*ToString()));
	return FText::Format(MatchingDescription[DescriptionIndex], Arguments);
}

FGameplayTagContainer FGameplayTag::GetSingleTagContainer() const
{
	SCOPE_CYCLE_COUNTER(STAT_FGameplayTag_GetSingleTagContainer);

	TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(*this);

	if (TagNode.IsValid())
	{
		return TagNode->GetSingleTagContainer();
	}

	// This tag should always be invalid if the node is missing
	ensureMsgf(!IsValid(), TEXT("GetSingleTagContainer passed invalid gameplay tag %s, only registered tags can be queried"), *GetTagName().ToString());

	return FGameplayTagContainer::EmptyContainer;
}

FGameplayTag FGameplayTag::RequestGameplayTag(const FName& TagName, bool ErrorIfNotFound)
{
	return UGameplayTagsManager::Get().RequestGameplayTag(TagName, ErrorIfNotFound);
}

bool FGameplayTag::IsValidGameplayTagString(const FString& TagString, FText* OutError, FString* OutFixedString)
{
	return UGameplayTagsManager::Get().IsValidGameplayTagString(TagString, OutError, OutFixedString);
}

FGameplayTagContainer FGameplayTag::GetGameplayTagParents() const
{
	return UGameplayTagsManager::Get().RequestGameplayTagParents(*this);
}

void FGameplayTag::ParseParentTags(TArray<FGameplayTag>& UniqueParentTags) const
{
	// This needs to be in the same order as the gameplay tag node ParentTags, which is immediate parent first
	FName RawTag = GetTagName();
	TStringBuilder<FName::StringBufferSize> TagBuffer(InPlace, RawTag);
	FStringView TagView = TagBuffer.ToView();

	int32 DotIndex = UE::String::FindLastChar(TagView, TEXT('.'));

	while (DotIndex != INDEX_NONE)
	{
		// Remove everything starting with the last dot
		TagView.LeftInline(DotIndex);
		DotIndex = UE::String::FindLastChar(TagView, TEXT('.'));

		// Add the name to the array
		FGameplayTag ParentTag = FGameplayTag(FName(TagView));

		UniqueParentTags.AddUnique(MoveTemp(ParentTag));
	}
}

FName FGameplayTag::GetTagLeafName() const
{
	FName RawTag = GetTagName();
	if (RawTag.IsNone())
	{
		return RawTag;
	}

	const TStringBuilder<FName::StringBufferSize> TagBuffer(InPlace, RawTag);
	FStringView TagView = TagBuffer.ToView();
	const int32 DotIndex = UE::String::FindLastChar(TagView, TEXT('.'));
	if (DotIndex == INDEX_NONE)
	{
		return RawTag;
	}

	TagView.RightChopInline(DotIndex + 1);
	return FName(TagView);
}

bool FGameplayTag::MatchesTag(const FGameplayTag& TagToCheck) const
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FGameplayTag_MatchesTag, GEnableGameplayTagDetailedStats);

	TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(*this);

	if (TagNode.IsValid())
	{
		return TagNode->GetSingleTagContainer().HasTag(TagToCheck);
	}

	// If a non-empty tag has not been registered, it will not exist in the tag database so this function may return the incorrect value
	// All tags must be registered from code or data before being used in matching functions and this tag may have been deleted with active references
	ensureMsgf(!IsValid(), TEXT("MatchesTag called on an invalid gameplay tag %s, only registered tags can be used in containers"), *GetTagName().ToString());

	return false;
}

bool FGameplayTag::MatchesAny(const FGameplayTagContainer& ContainerToCheck) const
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FGameplayTag_MatchesAny, GEnableGameplayTagDetailedStats);

	TSharedPtr<FGameplayTagNode> TagNode = UGameplayTagsManager::Get().FindTagNode(*this);

	if (TagNode.IsValid())
	{
		return TagNode->GetSingleTagContainer().HasAny(ContainerToCheck);
	}

	// If a non-empty tag has not been registered, it will not exist in the tag database so this function may return the incorrect value
	// All tags must be registered from code or data before being used in matching functions and this tag may have been deleted with active references
	ensureMsgf(!IsValid(), TEXT("MatchesAny called on an invalid gameplay tag %s, only registered tags can be used in containers"), *GetTagName().ToString());

	return false;
}

int32 FGameplayTag::MatchesTagDepth(const FGameplayTag& TagToCheck) const
{
	return UGameplayTagsManager::Get().GameplayTagsMatchDepth(*this, TagToCheck);
}

FGameplayTag::FGameplayTag(const FName& Name)
	: TagName(Name)
{
	// This constructor is used to bypass the table check and is only usable by GameplayTagManager
}

bool FGameplayTag::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		Slot << TagName;
		return true;
	}
	return false;
}

FGameplayTag FGameplayTag::RequestDirectParent() const
{
	return UGameplayTagsManager::Get().RequestGameplayTagDirectParent(*this);
}

bool FGameplayTag::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (Ar.IsSaving())
	{
		UGameplayTagsManager::Get().NotifyTagReplicated(*this, false);
	}
#endif

	NetSerialize_Packed(Ar, Map, bOutSuccess);

	bOutSuccess = true;
	return true;
}

static TSharedPtr<FNetFieldExportGroup> CreateNetfieldExportGroupForNetworkGameplayTags(const UGameplayTagsManager& TagManager, const TCHAR* NetFieldExportGroupName)
{
	TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = TSharedPtr<FNetFieldExportGroup>(new FNetFieldExportGroup());

	const TArray<TSharedPtr<FGameplayTagNode>>& NetworkGameplayTagNodeIndex = TagManager.GetNetworkGameplayTagNodeIndex();

	NetFieldExportGroup->PathName = NetFieldExportGroupName;
	NetFieldExportGroup->NetFieldExports.SetNum(NetworkGameplayTagNodeIndex.Num());

	for (int32 i = 0; i < NetworkGameplayTagNodeIndex.Num(); i++)
	{
		FNetFieldExport NetFieldExport(
			i,
			0,
			NetworkGameplayTagNodeIndex[i]->GetCompleteTagName());

		NetFieldExportGroup->NetFieldExports[i] = NetFieldExport;
	}

	return NetFieldExportGroup;
}

bool FGameplayTag::NetSerialize_ForReplayUsingFastReplication(FArchive& Ar, UPackageMapClient& PackageMapClient)
{
	UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
	FGameplayTagNetIndex NetIndex = INVALID_TAGNETINDEX;

	// For replays, use a net field export group to guarantee we can send the name reliably (without having to rely on the client having a deterministic NetworkGameplayTagNodeIndex array)
	const TCHAR* NetFieldExportGroupName = TEXT("NetworkGameplayTagNodeIndex");

	// Find this net field export group
	TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = PackageMapClient.GetNetFieldExportGroup(NetFieldExportGroupName);

	if (Ar.IsSaving())
	{
		// If we didn't find it, we need to create it (only when saving though, it should be here on load since it was exported at save time)
		if (!NetFieldExportGroup.IsValid())
		{
			NetFieldExportGroup = CreateNetfieldExportGroupForNetworkGameplayTags(TagManager, NetFieldExportGroupName);
			PackageMapClient.AddNetFieldExportGroup(NetFieldExportGroupName, NetFieldExportGroup);
		}

		NetIndex = TagManager.GetNetIndexFromTag(*this);

		if (NetIndex != TagManager.GetInvalidTagNetIndex() && NetIndex != INVALID_TAGNETINDEX)
		{
			PackageMapClient.TrackNetFieldExport(NetFieldExportGroup.Get(), NetIndex);
		}
		else
		{
			NetIndex = INVALID_TAGNETINDEX;		// We can't save InvalidTagNetIndex, since the remote side could have a different value for this
		}
	}

	uint32 NetIndex32 = NetIndex;
	Ar.SerializeIntPacked(NetIndex32);
	NetIndex = IntCastChecked<uint16, uint32>(NetIndex32);

	if (Ar.IsLoading())
	{
		// Get the tag name from the net field export group entry
		if (NetIndex != INVALID_TAGNETINDEX && ensure(NetFieldExportGroup.IsValid()) && ensure(NetIndex < NetFieldExportGroup->NetFieldExports.Num()))
		{
			TagName = NetFieldExportGroup->NetFieldExports[NetIndex].ExportName;

			// Validate the tag name
			const FGameplayTag Tag = TagManager.RequestGameplayTag(TagName, false);

			// Warn (once) if the tag isn't found
			if (!Tag.IsValid() && !NetFieldExportGroup->NetFieldExports[NetIndex].bIncompatible)
			{ 
				UE_LOGF(LogGameplayTags, Warning, "Gameplay tag not found (marking incompatible): %ls", *TagName.ToString());
				NetFieldExportGroup->NetFieldExports[NetIndex].bIncompatible = true;
			}

			TagName = Tag.TagName;
		}
		else
		{
			TagName = NAME_None;
		}
	}

	return true;
}

namespace UE::GameplayTags::GameplayTagDynamicSerialization
{
	// Can we make this generic and handle arbitrary export payloads? Probably something that should be handled in PackageMapClient
	static bool NetSerialize_ForReplay(FGameplayTag& GameplayTag, FArchive& Ar, UPackageMapClient& PackageMapClient)
	{
		using namespace UE::Net;

		const FNetTokenResolveContext* NetTokenResolveContext = PackageMapClient.GetNetTokenResolveContext();
		FGameplayTagTokenStore* TagTokenDataStore = NetTokenResolveContext ? NetTokenResolveContext->NetTokenStore->GetDataStore<UE::Net::FGameplayTagTokenStore>() : nullptr;
		if (!TagTokenDataStore)
		{
			return false;
		}

		const TCHAR* NetFieldExportGroupName = TEXT("NetworkGameplayTagDynamicIndex");
		TSharedPtr<FNetFieldExportGroup> NetFieldExportGroup = PackageMapClient.GetNetFieldExportGroup(NetFieldExportGroupName);
		FNetToken TagToken;

		if (Ar.IsSaving())
		{
			TagToken = TagTokenDataStore->GetOrCreateToken(GameplayTag);

			// Write token
			// Important: As we write it directly through the TagTokenStore we must read it in the same way as we skip serializing the type.
			TagTokenDataStore->WriteNetToken(Ar, TagToken);

			// Register replay export if needed
			if (TagToken.IsValid())
			{
				// If we didn't find it, we need to create it (only when saving though, it should be here on load since it was exported at save time)
				if (!NetFieldExportGroup.IsValid())
				{
					NetFieldExportGroup = TSharedPtr<FNetFieldExportGroup>(new FNetFieldExportGroup());
					NetFieldExportGroup->PathName = NetFieldExportGroupName;
					PackageMapClient.AddNetFieldExportGroup(NetFieldExportGroupName, NetFieldExportGroup);
				}

				// Make sure we have enough room in the NetFieldExports to hold this entry...
				const uint32 TagTokenIndex = TagToken.GetIndex();
				if (!NetFieldExportGroup->NetFieldExports.IsValidIndex(TagTokenIndex))
				{
					NetFieldExportGroup->NetFieldExports.SetNum(TagTokenIndex + 1, EAllowShrinking::No);
					NetFieldExportGroup->bDirtyForReplay = true;
				}
				ensure(NetFieldExportGroup->NetFieldExports.IsValidIndex(TagTokenIndex));
				FNetFieldExport& NetFieldExport = NetFieldExportGroup->NetFieldExports[TagTokenIndex];

				// If it's not yet exported, export it now
				if (!NetFieldExport.bExported)
				{
					NetFieldExport = FNetFieldExport(TagTokenIndex, 0, GameplayTag.GetTagName());
					UE_LOGF(LogGameplayTags, Verbose, "Replay> Exported Tag %ls as NetFieldIndex %u", *NetFieldExport.ExportName.ToString(), TagTokenIndex);
				}

				// Track the export so that it gets added to the replay index
				PackageMapClient.TrackNetFieldExport(NetFieldExportGroup.Get(), TagTokenIndex);
			}
			return true;
		}
		else if (Ar.IsLoading())
		{
			// Read TagNetToken
			TagToken = TagTokenDataStore->ReadNetToken(Ar);

			if (Ar.IsError())
			{
				return false;
			}

			if (TagToken.IsValid())
			{
				const uint32 TagTokenIndex = TagToken.GetIndex();
				if (ensure(NetFieldExportGroup.IsValid()) && ensure(TagTokenIndex < (uint32)NetFieldExportGroup->NetFieldExports.Num()))
				{
					FName TagName = NetFieldExportGroup->NetFieldExports[TagTokenIndex].ExportName;

					// Validate the tag name
					// TODO: Should we be able to add tags through this?
					UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
					GameplayTag = TagManager.RequestGameplayTag(TagName, false);

					// Warn (once) if the tag isn't found
					if (!GameplayTag.IsValid() && !NetFieldExportGroup->NetFieldExports[TagTokenIndex].bIncompatible)
					{ 
						UE_LOGF(LogGameplayTags, Warning, "Gameplay tag not found (marking incompatible): %ls", *TagName.ToString());
						NetFieldExportGroup->NetFieldExports[TagTokenIndex].bIncompatible = true;
					}
					return true;
				}
				else
				{
					GameplayTag = FGameplayTag();
					return false;
				}
			}
			GameplayTag = FGameplayTag();
			return true;
		}
		return false;
	}

	bool NetSerialize(FGameplayTag& GameplayTag, FArchive& Ar, UPackageMap* Map)
	{
		using namespace UE::Net;

		// For now special case replays
		UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Map);
		if (const bool bIsReplay = PackageMapClient && PackageMapClient->GetConnection() && PackageMapClient->GetConnection()->IsInternalAck())
		{
			return NetSerialize_ForReplay(GameplayTag, Ar, *PackageMapClient);
		}
	
		if (Ar.IsSaving())
		{
			UE::Net::FNetTokenExportContext* ExportContext = FNetTokenExportContext::GetNetTokenExportContext(Ar);
			UE::Net::FNetTokenStore* NetTokenStore = ExportContext ? ExportContext->GetNetTokenStore() : nullptr;
			FGameplayTagTokenStore* TagTokenStore = NetTokenStore ? NetTokenStore->GetDataStore<UE::Net::FGameplayTagTokenStore>() : nullptr;
			if (ensure(TagTokenStore))
			{
				FNetToken TagToken;
				if (GameplayTag.IsValid())
				{
					TagToken = TagTokenStore->GetOrCreateToken(GameplayTag);
				}

				UE_NET_TRACE_DYNAMIC_NAME_SCOPE(*TagToken.ToString(), static_cast<FNetBitWriter&>(Ar), GetTraceCollector(static_cast<FNetBitWriter&>(Ar)), ENetTraceVerbosity::VeryVerbose);

				// Write NetToken, 
				// Important: As we write it directly thorugh the TagTokenStore we also need to read it in the samw way as we skip serializing the type.
				TagTokenStore->WriteNetToken(Ar, TagToken);

				// Add export
				ExportContext->AddNetTokenPendingExport(TagToken);

				return true;
			}
			else
			{
				UE_LOGF(LogGameplayTags, Error, "FGameplayTag::NetSerialize::Could not find required FGameplayTagTokenStore");
				ensure(false);
			}
		}
		else if (Ar.IsLoading())
		{
			// When reading data we always have a PackageMap so we can get the necessary resolve context from here.
			const FNetTokenResolveContext* NetTokenResolveContext = Map ? Map->GetNetTokenResolveContext() : nullptr;
			FGameplayTagTokenStore* TagTokenStore = NetTokenResolveContext ? NetTokenResolveContext->NetTokenStore->GetDataStore<UE::Net::FGameplayTagTokenStore>() : nullptr;
			if (ensure(TagTokenStore))
			{
				// Read the TagToken using the TagTokenStore
				FNetToken TagToken = TagTokenStore->ReadNetToken(Ar);
				if (Ar.IsError())
				{
					return false;
				}

				// Resolve the TagToken
				GameplayTag = TagTokenStore->ResolveToken(TagToken, NetTokenResolveContext->RemoteNetTokenStoreState);
				return true;
			}
			else
			{
				UE_LOGF(LogGameplayTags, Error, "FGameplayTag::NetSerialize::Could not find required FGameplayTagTokenStore");	
				ensure(false);
				Ar.SetError();
			}
		}
		return false;
	}
}


bool FGameplayTag::NetSerialize_Packed(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	CONDITIONAL_SCOPE_CYCLE_COUNTER(STAT_FGameplayTag_NetSerialize, GEnableGameplayTagDetailedStats);
	UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();

	// We now save off which method we're using (below)
	bool bUseFastReplication = TagManager.ShouldUseFastReplication();
	bool bUseDynamicReplication = TagManager.ShouldUseDynamicReplication();

	Ar.UsingCustomVersion(FEngineNetworkCustomVersion::Guid);
	const bool bSerializeReplicationMethod = (Ar.EngineNetVer() >= FEngineNetworkCustomVersion::CustomExports);
	if (bSerializeReplicationMethod)
	{
		Ar.SerializeBits(&bUseFastReplication, 1);
		if (!bUseFastReplication)
		{
			Ar.SerializeBits(&bUseDynamicReplication, 1);
		}
	}
	else
	{
		bUseFastReplication = GOldReplaysUseGameplayTagFastReplication;
		bUseDynamicReplication = false; // this didn't exist in prior versions
	}

	if (bUseFastReplication)
	{
		UPackageMapClient* PackageMapClient = Cast<UPackageMapClient>(Map);

		const bool bIsReplay = PackageMapClient && PackageMapClient->GetConnection() && PackageMapClient->GetConnection()->IsInternalAck();
		if (bIsReplay)
		{
			return NetSerialize_ForReplayUsingFastReplication(Ar, *PackageMapClient);
		}

		FGameplayTagNetIndex NetIndex = INVALID_TAGNETINDEX;
		if (Ar.IsSaving())
		{
			NetIndex = TagManager.GetNetIndexFromTag(*this);
			SerializeTagNetIndexPacked(Ar, NetIndex, TagManager.GetNetIndexFirstBitSegment(), TagManager.GetNetIndexTrueBitNum());
		}
		else
		{
			SerializeTagNetIndexPacked(Ar, NetIndex, TagManager.GetNetIndexFirstBitSegment(), TagManager.GetNetIndexTrueBitNum());
			TagName = TagManager.GetTagNameFromNetIndex(NetIndex);
		}
	}
	else if (bUseDynamicReplication)
	{
		UE::GameplayTags::GameplayTagDynamicSerialization::NetSerialize(*this, Ar, Map);
	}
	else
	{
		Ar << TagName;
	}

	bOutSuccess = true;
	return true;
}

void FGameplayTag::PostSerialize(const FArchive& Ar)
{
	// This only happens for tags that are not nested inside a container, containers handle redirectors themselves
	// Only do redirects for real loads, not for duplicates or recompiles
	if (Ar.IsLoading() && Ar.IsPersistent() && !(Ar.GetPortFlags() & PPF_Duplicate) && !(Ar.GetPortFlags() & PPF_DuplicateForPIE))
	{
		// Rename any tags that may have changed by the ini file.
		UGameplayTagsManager::Get().SingleGameplayTagLoaded(*this, Ar.GetSerializedProperty());
	}

	if (Ar.IsSaving())
	{
		if (IsValid())
		{
			// This marks the saved name for later searching
			Ar.MarkSearchableName(FGameplayTag::StaticStruct(), TagName);
		}
#if WITH_EDITOR
		if (Ar.IsCooking())
		{
			FArchiveSavePackageData* SavePackageData = const_cast<FArchive&>(Ar).GetSavePackageData();
			check(SavePackageData);
			FObjectSavePackageSerializeContext& SaveContext = SavePackageData->SavePackageContext;

			if (SaveContext.IsHarvestingCookDependencies())
			{
				SaveContext.AddCookLoadDependency(UGameplayTagsManager::Get().CreateCookDependency(*this));
			}
		}
#endif //#if WITH_EDITOR
	}
}

bool FGameplayTag::ImportTextItem(const TCHAR*& Buffer, int32 PortFlags, UObject* Parent, FOutputDevice* ErrorText)
{
	FString ImportedTag;
	const TCHAR* NewBuffer = FPropertyHelpers::ReadToken(Buffer, ImportedTag, true);
	if (!NewBuffer)
	{
		// Failed to read buffer. Maybe normal ImportText will work.
		return false;
	}
	
	const TCHAR* OriginalBuffer = Buffer;
	Buffer = NewBuffer;

	if (ImportedTag == TEXT("None") || ImportedTag.IsEmpty())
	{
		// TagName was none
		TagName = NAME_None;
		return true;
	}

	if (ImportedTag[0] == '(')
	{
		// Let normal ImportText handle this before handling fixups
		UScriptStruct* ScriptStruct = FGameplayTag::StaticStruct();
		Buffer = ScriptStruct->ImportText(OriginalBuffer, this, Parent, PortFlags, ErrorText, ScriptStruct->GetName(), false);
		UGameplayTagsManager::Get().ImportSingleGameplayTag(*this, TagName, !!(PortFlags & PPF_SerializedAsImportText));
		return true;
	}

	return UGameplayTagsManager::Get().ImportSingleGameplayTag(*this, FName(*ImportedTag), !!(PortFlags & PPF_SerializedAsImportText));
}

void FGameplayTag::FromExportString(const FString& ExportString, int32 PortFlags)
{
	TagName = NAME_None;

	FOutputDeviceNull NullOut;
	FGameplayTag::StaticStruct()->ImportText(*ExportString, this, nullptr, PortFlags, &NullOut, TEXT("FGameplayTag"), true);
}

FGameplayTagNativeAdder::FGameplayTagNativeAdder()
{
	AddTagsHandle = UGameplayTagsManager::CallOrRegister_OnAddNativeTagsDelegate(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FGameplayTagNativeAdder::AddTags));
}

FGameplayTagNativeAdder::~FGameplayTagNativeAdder()
{
	UGameplayTagsManager::UnregisterNativeTagDelegate(AddTagsHandle);
}

FGameplayTagQuery::FGameplayTagQuery()
	: TokenStreamVersion(EGameplayTagQueryStreamVersion::LatestVersion)
{
}

FGameplayTagQuery::FGameplayTagQuery(FGameplayTagQuery const& Other)
{
	*this = Other;
}

FGameplayTagQuery::FGameplayTagQuery(FGameplayTagQuery&& Other)
{
	*this = MoveTemp(Other);
}

/** Assignment/Equality operators */
FGameplayTagQuery& FGameplayTagQuery::operator=(FGameplayTagQuery const& Other)
{
	if (this != &Other)
	{
		TokenStreamVersion = Other.TokenStreamVersion;
		TagDictionary = Other.TagDictionary;
		QueryTokenStream = Other.QueryTokenStream;
		UserDescription = Other.UserDescription;
		AutoDescription = Other.AutoDescription;
	}
	return *this;
}

FGameplayTagQuery& FGameplayTagQuery::operator=(FGameplayTagQuery&& Other)
{
	TokenStreamVersion = Other.TokenStreamVersion;
	TagDictionary = MoveTemp(Other.TagDictionary);
	QueryTokenStream = MoveTemp(Other.QueryTokenStream);
	UserDescription = MoveTemp(Other.UserDescription);
	AutoDescription = MoveTemp(Other.AutoDescription);
	return *this;
}

bool FGameplayTagQuery::operator==(const FGameplayTagQuery& Other) const
{
	return TokenStreamVersion == Other.TokenStreamVersion &&
		TagDictionary == Other.TagDictionary &&
		QueryTokenStream == Other.QueryTokenStream &&
		UserDescription == Other.UserDescription &&
		AutoDescription == Other.AutoDescription;
}

bool FGameplayTagQuery::operator!=(const FGameplayTagQuery& Other) const
{
	return !(*this == Other);
}

bool FGameplayTagQuery::Matches(FGameplayTagContainer const& Tags) const
{
	if (IsEmpty())
	{
		return false;
	}

	FQueryEvaluator QE(*this);
	return QE.Eval(Tags);
}

bool FGameplayTagQuery::IsEmpty() const
{
	return (QueryTokenStream.Num() == 0);
}

void FGameplayTagQuery::Clear()
{
	*this = FGameplayTagQuery::EmptyQuery;
}

void FGameplayTagQuery::GetQueryExpr(FGameplayTagQueryExpression& OutExpr) const
{
	// build the FExpr tree from the token stream and return it
	FQueryEvaluator QE(*this);
	QE.Read(OutExpr);
}

const TArray<FGameplayTag>& FGameplayTagQuery::GetGameplayTagArray() const
{
	return TagDictionary;
}
	
void FGameplayTagQuery::Serialize(FArchive& Ar)
{
	Ar << TokenStreamVersion;
	Ar << TagDictionary;
	Ar << QueryTokenStream;

#if WITH_EDITOR
	if (!Ar.IsCooking())
	{
		// Descriptions are not needed during runtime
		Ar << UserDescription;
		Ar << AutoDescription;
	}
#endif
}

void FGameplayTagQuery::Build(FGameplayTagQueryExpression& RootQueryExpr, FString InUserDescription)
{
	TokenStreamVersion = EGameplayTagQueryStreamVersion::LatestVersion;
	UserDescription = InUserDescription;

	// Reserve size here is arbitrary, goal is to minimizing reallocs while being respectful of mem usage
	QueryTokenStream.Reset(128);
	TagDictionary.Reset();

	// add stream version first
	QueryTokenStream.Add(EGameplayTagQueryStreamVersion::LatestVersion);

	// emit the query
	QueryTokenStream.Add(1);		// true to indicate is has a root expression
	RootQueryExpr.EmitTokens(QueryTokenStream, TagDictionary);
}

// static 
FGameplayTagQuery FGameplayTagQuery::BuildQuery(FGameplayTagQueryExpression& RootQueryExpr, FString InDescription)
{
	FGameplayTagQuery Q;
	Q.Build(RootQueryExpr, InDescription);
	return Q;
}

//static 
FGameplayTagQuery FGameplayTagQuery::MakeQuery_MatchAnyTags(FGameplayTagContainer const& InTags)
{
	return FGameplayTagQuery::BuildQuery
	(
		FGameplayTagQueryExpression()
		.AnyTagsMatch()
		.AddTags(InTags)
	);
}

//static
FGameplayTagQuery FGameplayTagQuery::MakeQuery_MatchAllTags(FGameplayTagContainer const& InTags)
{
	return FGameplayTagQuery::BuildQuery
		(
		FGameplayTagQueryExpression()
		.AllTagsMatch()
		.AddTags(InTags)
		);
}

// static
FGameplayTagQuery FGameplayTagQuery::MakeQuery_MatchNoTags(FGameplayTagContainer const& InTags)
{
	return FGameplayTagQuery::BuildQuery
		(
		FGameplayTagQueryExpression()
		.NoTagsMatch()
		.AddTags(InTags)
		);
}

// static
FGameplayTagQuery FGameplayTagQuery::MakeQuery_ExactMatchAnyTags(FGameplayTagContainer const& InTags)
{
	return FGameplayTagQuery::BuildQuery
		(
		FGameplayTagQueryExpression()
		.AnyTagsExactMatch()
		.AddTags(InTags)
		);
}

// static
FGameplayTagQuery FGameplayTagQuery::MakeQuery_ExactMatchAllTags(FGameplayTagContainer const& InTags)
{
	return FGameplayTagQuery::BuildQuery
		(
		FGameplayTagQueryExpression()
		.AllTagsExactMatch()
		.AddTags(InTags)
		);
}

// static
FGameplayTagQuery FGameplayTagQuery::MakeQuery_MatchTag(FGameplayTag const & InTag)
{
	return FGameplayTagQuery::BuildQuery
	(
		FGameplayTagQueryExpression()
		.AllTagsMatch()
		.AddTag(InTag)
	);
}


#if WITH_EDITOR

UEditableGameplayTagQuery* FQueryEvaluator::CreateEditableQuery()
{
	CurStreamIdx = 0;

	UEditableGameplayTagQuery* const EditableQuery = NewObject<UEditableGameplayTagQuery>(GetTransientPackage(), NAME_None, RF_Transactional);

	// start parsing the set
	if (!Query.IsEmpty())
	{
		Version = GetToken();
		if (!bReadError)
		{
			uint8 const bHasRootExpression = GetToken();
			if (!bReadError && bHasRootExpression)
			{
				EditableQuery->RootExpression = ReadEditableQueryExpr(EditableQuery);
			}
		}
		ensure(CurStreamIdx == Query.QueryTokenStream.Num());
	}

	EditableQuery->UserDescription = Query.UserDescription;

	return EditableQuery;
}

UEditableGameplayTagQueryExpression* FQueryEvaluator::ReadEditableQueryExpr(UObject* ExprOuter)
{
	EGameplayTagQueryExprType const ExprType = (EGameplayTagQueryExprType) GetToken();
	if (bReadError)
	{
		return nullptr;
	}

	UClass* ExprClass = nullptr;
	switch (ExprType)
	{
	case EGameplayTagQueryExprType::AnyTagsMatch:
		ExprClass = UEditableGameplayTagQueryExpression_AnyTagsMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::AllTagsMatch:
		ExprClass = UEditableGameplayTagQueryExpression_AllTagsMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::NoTagsMatch:
		ExprClass = UEditableGameplayTagQueryExpression_NoTagsMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::AnyExprMatch:
		ExprClass = UEditableGameplayTagQueryExpression_AnyExprMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::AllExprMatch:
		ExprClass = UEditableGameplayTagQueryExpression_AllExprMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::NoExprMatch:
		ExprClass = UEditableGameplayTagQueryExpression_NoExprMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::AnyTagsExactMatch:
		ExprClass = UEditableGameplayTagQueryExpression_AnyTagsExactMatch::StaticClass();
		break;
	case EGameplayTagQueryExprType::AllTagsExactMatch:
		ExprClass = UEditableGameplayTagQueryExpression_AllTagsExactMatch::StaticClass();
		break;
	}

	UEditableGameplayTagQueryExpression* NewExpr = nullptr;
	if (ExprClass)
	{
		NewExpr = NewObject<UEditableGameplayTagQueryExpression>(ExprOuter, ExprClass, NAME_None, RF_Transactional);
		if (NewExpr)
		{
			switch (ExprType)
			{
			case EGameplayTagQueryExprType::AnyTagsMatch:
			case EGameplayTagQueryExprType::AllTagsMatch:
			case EGameplayTagQueryExprType::NoTagsMatch:
			case EGameplayTagQueryExprType::AnyTagsExactMatch:
			case EGameplayTagQueryExprType::AllTagsExactMatch:
				ReadEditableQueryTags(NewExpr);
				break;
			case EGameplayTagQueryExprType::AnyExprMatch:
			case EGameplayTagQueryExprType::AllExprMatch:
			case EGameplayTagQueryExprType::NoExprMatch:
				ReadEditableQueryExprList(NewExpr);
				break;
			}
		}
	}

	return NewExpr;
}

void FQueryEvaluator::ReadEditableQueryTags(UEditableGameplayTagQueryExpression* EditableQueryExpr)
{
	// find the tag container to read into
	FGameplayTagContainer* Tags = nullptr;
	if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_AnyTagsMatch::StaticClass()))
	{
		Tags = &((UEditableGameplayTagQueryExpression_AnyTagsMatch*)EditableQueryExpr)->Tags;
	}
	else if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_AllTagsMatch::StaticClass()))
	{
		Tags = &((UEditableGameplayTagQueryExpression_AllTagsMatch*)EditableQueryExpr)->Tags;
	}
	else if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_NoTagsMatch::StaticClass()))
	{
		Tags = &((UEditableGameplayTagQueryExpression_NoTagsMatch*)EditableQueryExpr)->Tags;
	}
	else if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_AnyTagsExactMatch::StaticClass()))
	{
		Tags = &((UEditableGameplayTagQueryExpression_AnyTagsExactMatch*)EditableQueryExpr)->Tags;
	}
	else if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_AllTagsExactMatch::StaticClass()))
	{
		Tags = &((UEditableGameplayTagQueryExpression_AllTagsExactMatch*)EditableQueryExpr)->Tags;
	}
	ensure(Tags);

	if (Tags)
	{
		// parse tag set
		int32 const NumTags = GetToken();
		if (bReadError)
		{
			return;
		}

		for (int32 Idx = 0; Idx < NumTags; ++Idx)
		{
			int32 const TagIdx = GetToken();
			if (bReadError)
			{
				return;
			}

			FGameplayTag const Tag = Query.GetTagFromIndex(TagIdx);
			Tags->AddTag(Tag);
		}
	}
}

void FQueryEvaluator::ReadEditableQueryExprList(UEditableGameplayTagQueryExpression* EditableQueryExpr)
{
	// find the tag container to read into
	using ExpressionArrayType = decltype(UEditableGameplayTagQueryExpression_AnyExprMatch::Expressions);
	ExpressionArrayType* ExprList = nullptr;
	if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_AnyExprMatch::StaticClass()))
	{
		ExprList = &((UEditableGameplayTagQueryExpression_AnyExprMatch*)EditableQueryExpr)->Expressions;
	}
	else if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_AllExprMatch::StaticClass()))
	{
		ExprList = &((UEditableGameplayTagQueryExpression_AllExprMatch*)EditableQueryExpr)->Expressions;
	}
	else if (EditableQueryExpr->IsA(UEditableGameplayTagQueryExpression_NoExprMatch::StaticClass()))
	{
		ExprList = &((UEditableGameplayTagQueryExpression_NoExprMatch*)EditableQueryExpr)->Expressions;
	}
	ensure(ExprList);

	if (ExprList)
	{
		// parse expr set
		int32 const NumExprs = GetToken();
		if (bReadError)
		{
			return;
		}

		for (int32 Idx = 0; Idx < NumExprs; ++Idx)
		{
			UEditableGameplayTagQueryExpression* const NewExpr = ReadEditableQueryExpr(EditableQueryExpr);
			ExprList->Add(NewExpr);
		}
	}
}

UEditableGameplayTagQuery* FGameplayTagQuery::CreateEditableQuery() const
{
	FQueryEvaluator QE(*this);
	return QE.CreateEditableQuery();
}

void FGameplayTagQuery::BuildFromEditableQuery(UEditableGameplayTagQuery& EditableQuery)
{
	QueryTokenStream.Reset();
	TagDictionary.Reset();

	UserDescription = EditableQuery.UserDescription;

	// add stream version first
	QueryTokenStream.Add(EGameplayTagQueryStreamVersion::LatestVersion);
	EditableQuery.EmitTokens(QueryTokenStream, TagDictionary, &AutoDescription);
}

FString UEditableGameplayTagQuery::GetTagQueryExportText(FGameplayTagQuery const& TagQuery)
{
	TagQueryExportText_Helper = TagQuery;
	FProperty* const TQProperty = FindFProperty<FProperty>(GetClass(), TEXT("TagQueryExportText_Helper"));

	FString OutString;
	TQProperty->ExportTextItem_Direct(OutString, (void*)&TagQueryExportText_Helper, (void*)&TagQueryExportText_Helper, this, 0);
	return OutString;
}

void UEditableGameplayTagQuery::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	if (DebugString)
	{
		// start with a fresh string
		DebugString->Empty();
	}

	if (RootExpression)
	{
		TokenStream.Add(1);		// true if has a root expression
		RootExpression->EmitTokens(TokenStream, TagDictionary, DebugString);
	}
	else
	{
		TokenStream.Add(0);		// false if no root expression
		if (DebugString)
		{
			DebugString->Append(TEXT("undefined"));
		}
	}
}

void UEditableGameplayTagQueryExpression::EmitTagTokens(FGameplayTagContainer const& TagsToEmit, TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	uint8 const NumTags = (uint8)TagsToEmit.Num();
	TokenStream.Add(NumTags);

	bool bFirstTag = true;

	for (auto T : TagsToEmit)
	{
		int32 TagIdx = TagDictionary.AddUnique(T);
		check(TagIdx <= 255);
		TokenStream.Add((uint8)TagIdx);

		if (DebugString)
		{
			if (bFirstTag == false)
			{
				DebugString->Append(TEXT(","));
			}

			DebugString->Append(TEXT(" "));
			DebugString->Append(T.ToString());
		}

		bFirstTag = false;
	}
}

void UEditableGameplayTagQueryExpression::EmitExprListTokens(TArray<UEditableGameplayTagQueryExpression*> const& ExprList, TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	uint8 const NumExprs = (uint8)ExprList.Num();
	TokenStream.Add(NumExprs);

	bool bFirstExpr = true;
	
	for (auto E : ExprList)
	{
		if (DebugString)
		{
			if (bFirstExpr == false)
			{
				DebugString->Append(TEXT(","));
			}

			DebugString->Append(TEXT(" "));
		}

		if (E)
		{
			E->EmitTokens(TokenStream, TagDictionary, DebugString);
		}
		else
		{
			// null expression
			TokenStream.Add((uint8)EGameplayTagQueryExprType::Undefined);
			if (DebugString)
			{
				DebugString->Append(TEXT("undefined"));
			}
		}

		bFirstExpr = false;
	}
}

void UEditableGameplayTagQueryExpression_AnyTagsMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::AnyTagsMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" ANY("));
	}

	EmitTagTokens(Tags, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_AllTagsMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::AllTagsMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" ALL("));
	}
	
	EmitTagTokens(Tags, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_NoTagsMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::NoTagsMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" NONE("));
	}

	EmitTagTokens(Tags, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_AnyTagsExactMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary,
	FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::AnyTagsExactMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" ANY_EXACT("));
	}
	
	EmitTagTokens(Tags, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_AllTagsExactMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary,
	FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::AllTagsExactMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" ALL_EXACT("));
	}
	
	EmitTagTokens(Tags, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_AnyExprMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::AnyExprMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" ANY("));
	}

	EmitExprListTokens(Expressions, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_AllExprMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::AllExprMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" ALL("));
	}

	EmitExprListTokens(Expressions, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}

void UEditableGameplayTagQueryExpression_NoExprMatch::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary, FString* DebugString) const
{
	TokenStream.Add((uint8)EGameplayTagQueryExprType::NoExprMatch);

	if (DebugString)
	{
		DebugString->Append(TEXT(" NONE("));
	}

	EmitExprListTokens(Expressions, TokenStream, TagDictionary, DebugString);

	if (DebugString)
	{
		DebugString->Append(TEXT(" )"));
	}
}
#endif	// WITH_EDITOR


FGameplayTagQueryExpression& FGameplayTagQueryExpression::AddTag(FName TagName)
{
	FGameplayTag const Tag = UGameplayTagsManager::Get().RequestGameplayTag(TagName);
	return AddTag(Tag);
}

void FGameplayTagQueryExpression::EmitTokens(TArray<uint8>& TokenStream, TArray<FGameplayTag>& TagDictionary) const
{
	// emit exprtype
	TokenStream.Add(static_cast<uint8>(ExprType));

	// emit exprdata
	switch (ExprType)
	{
	case EGameplayTagQueryExprType::AnyTagsMatch:
	case EGameplayTagQueryExprType::AllTagsMatch:
	case EGameplayTagQueryExprType::NoTagsMatch:
	case EGameplayTagQueryExprType::AnyTagsExactMatch:
	case EGameplayTagQueryExprType::AllTagsExactMatch:
	{
		// emit tagset
		uint8 NumTags = (uint8)TagSet.Num();
		TokenStream.Add(NumTags);

		for (auto Tag : TagSet)
		{
			int32 TagIdx = TagDictionary.AddUnique(Tag);
			check(TagIdx <= 254);		// we reserve token 255 for internal use, so 254 is max unique tags
			TokenStream.Add((uint8)TagIdx);
		}
	}
	break;

	case EGameplayTagQueryExprType::AnyExprMatch:
	case EGameplayTagQueryExprType::AllExprMatch:
	case EGameplayTagQueryExprType::NoExprMatch:
	{
		// emit tagset
		uint8 NumExprs = (uint8)ExprSet.Num();
		TokenStream.Add(NumExprs);

		for (auto& E : ExprSet)
		{
			E.EmitTokens(TokenStream, TagDictionary);
		}
	}
	break;
	default:
		break;
	}
}

bool FGameplayTagQueryExpression::ConvertToJsonObject(TSharedRef<FJsonObject>& OutObject) const
{
	TArray<TSharedPtr<FJsonValue>> TagSetArray;
	TArray<TSharedPtr<FJsonValue>> ExprSetArray;
	for (const FGameplayTag& GameplayTag : TagSet)
	{
		TSharedRef<FJsonObject> TagJsonObject = MakeShared<FJsonObject>();
		if (FJsonObjectConverter::UStructToJsonObject(FGameplayTag::StaticStruct(), &GameplayTag, TagJsonObject))
		{
			TagSetArray.Add(MakeShared<FJsonValueObject>(TagJsonObject).ToSharedPtr());
		}
		else
		{
			UE_LOGF(LogGameplayTags, Error, "Failed to convert GameplayTag (%ls) to json", *GameplayTag.ToString());
			return false;
		}
	}
	for (const FGameplayTagQueryExpression& TagQueryExpression : ExprSet)
	{
		TSharedRef<FJsonObject> ExprJsonObject = MakeShared<FJsonObject>();
		if (TagQueryExpression.ConvertToJsonObject(ExprJsonObject)) // caution recursion
		{
			ExprSetArray.Add(MakeShared<FJsonValueObject>(ExprJsonObject).ToSharedPtr());
		}
		else
		{
			UE_LOGF(LogGameplayTags, Error, "Failed to convert tag query expression to json");
			return false;
		}
	}

	OutObject->SetStringField(TEXT("ExprType"), UEnum::GetValueAsString(ExprType));
	OutObject->SetArrayField(TEXT("ExprSet"), ExprSetArray);
	OutObject->SetArrayField(TEXT("TagSet"), TagSetArray);
	return true;
}

bool FGameplayTagQueryExpression::MakeFromJsonObject(const TSharedRef<FJsonObject>& InObject, FGameplayTagQueryExpression& OutQueryExpression)
{
	FGameplayTagQueryExpression ReturnQueryExpression;

	FString FoundExprType;
	if (InObject->TryGetStringField(TEXT("ExprType"), FoundExprType))
	{
		ReturnQueryExpression.ExprType = (EGameplayTagQueryExprType)StaticEnum<EGameplayTagQueryExprType>()->GetValueByNameString(FoundExprType);
	}
	else
	{
		UE_LOGF(LogGameplayTags, Error, "Failed to get ExprType from json object. Unable to make GameplayTagQueryExpression");
		return false;
	}

	{
		const TArray<TSharedPtr<FJsonValue>>* TagSetArray;
		if (InObject->TryGetArrayField(TEXT("TagSet"), TagSetArray))
		{
			for (const TSharedPtr<FJsonValue>& GameplayTagObject : *TagSetArray)
			{
				FGameplayTag GameplayTag;
				if (FJsonObjectConverter::JsonObjectToUStruct(GameplayTagObject->AsObject().ToSharedRef(), FGameplayTag::StaticStruct(), &GameplayTag))
				{
					ReturnQueryExpression.TagSet.Add(GameplayTag);
				}
				else
				{
					UE_LOGF(LogGameplayTags, Error, "Failed to make GameplayTag from Json");
					return false;
				}
			}
		}
	}
	{
		const TArray<TSharedPtr<FJsonValue>>* ExprSetArray;
		if (InObject->TryGetArrayField(TEXT("ExprSet"), ExprSetArray))
		{
			for (const TSharedPtr<FJsonValue>& ExprJsonObject : *ExprSetArray)
			{
				FGameplayTagQueryExpression TagQueryExpression;
				if (MakeFromJsonObject(ExprJsonObject->AsObject().ToSharedRef(), TagQueryExpression))
				{
					ReturnQueryExpression.ExprSet.Add(TagQueryExpression);
				}
				else
				{
					UE_LOGF(LogGameplayTags, Error, "Failed to make GameplayTagQueryExpression from Json");
					return false;
				}
			}
		}
	}

	OutQueryExpression = MoveTemp(ReturnQueryExpression);
	return true;
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

static void GameplayTagPrintReplicationMap()
{
	UGameplayTagsManager::Get().PrintReplicationFrequencyReport();
}

FAutoConsoleCommand GameplayTagPrintReplicationMapCmd(
	TEXT("GameplayTags.PrintReport"), 
	TEXT( "Prints frequency of gameplay tags" ), 
	FConsoleCommandDelegate::CreateStatic(GameplayTagPrintReplicationMap)
);

static void GameplayTagPrintReplicationIndices()
{
	UGameplayTagsManager::Get().PrintReplicationIndices();
}

FAutoConsoleCommand GameplayTagPrintReplicationIndicesCmd(
	TEXT("GameplayTags.PrintNetIndices"), 
	TEXT( "Prints net indices for all known tags" ), 
	FConsoleCommandDelegate::CreateStatic(GameplayTagPrintReplicationIndices)
);

static void TagPackingTest()
{
	for (int32 TotalNetIndexBits=1; TotalNetIndexBits <= 16; ++TotalNetIndexBits)
	{
		for (int32 NetIndexBitsPerComponent=0; NetIndexBitsPerComponent <= TotalNetIndexBits; NetIndexBitsPerComponent++)
		{
			for (int32 NetIndex=0; NetIndex < FMath::Pow(2.f, TotalNetIndexBits); ++NetIndex)
			{
				FGameplayTagNetIndex NI = IntCastChecked<uint16, int32>(NetIndex);

				FNetBitWriter	BitWriter(nullptr, 1024 * 8);
				SerializeTagNetIndexPacked(BitWriter, NI, NetIndexBitsPerComponent, TotalNetIndexBits);

				FNetBitReader	Reader(nullptr, BitWriter.GetData(), BitWriter.GetNumBits());

				FGameplayTagNetIndex NewIndex;
				SerializeTagNetIndexPacked(Reader, NewIndex, NetIndexBitsPerComponent, TotalNetIndexBits);

				if (ensureAlways((NewIndex == NI)) == false)
				{
					NetIndex--;
					continue;
				}
			}
		}
	}

	UE_LOGF(LogGameplayTags, Warning, "TagPackingTest completed!");

}

FAutoConsoleCommand TagPackingTestCmd(
	TEXT("GameplayTags.PackingTest"), 
	TEXT( "Prints frequency of gameplay tags" ), 
	FConsoleCommandDelegate::CreateStatic(TagPackingTest)
);

#endif

