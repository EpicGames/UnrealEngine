// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagContainerNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagContainerNetSerializer)


#include "InternalGameplayTagContainerNetSerializer.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/ReplicationState/ReplicationStateDescriptorBuilder.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/Serialization/NetSerializers.h"

namespace UE::Net
{

struct FGameplayTagContainerAccessorForNetSerializer : public FGameplayTagContainer
{
	const TArray<FGameplayTag>& GetGameplayTags() const { return GameplayTags; }
	TArray<FGameplayTag>& GetGameplayTags() { return GameplayTags; }
};

struct FGameplayTagContainerNetSerializer
{
	// Version
	static const uint32 Version = 0;

	// Traits
	static constexpr bool bIsForwardingSerializer = true; // Triggers asserts if a function is missing
	static constexpr bool bHasDynamicState = true;

	// Types
	struct FQuantizedType
	{
		alignas(8) uint8 QuantizedStruct[16];
	};

	typedef FGameplayTagContainerAccessorForNetSerializer SourceType;
	typedef FQuantizedType QuantizedType;
	typedef FGameplayTagContainerNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs& Args);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs& Args);

	static void SerializeDelta(FNetSerializationContext&, const FNetSerializeDeltaArgs& Args);
	static void DeserializeDelta(FNetSerializationContext&, const FNetDeserializeDeltaArgs& Args);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs& Args);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs& Args);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs& Args);
	static bool Validate(FNetSerializationContext&, const FNetValidateArgs& Args);

	static void CloneDynamicState(FNetSerializationContext&, const FNetCloneDynamicStateArgs&);
	static void FreeDynamicState(FNetSerializationContext&, const FNetFreeDynamicStateArgs&);

	static void CollectNetReferences(FNetSerializationContext&, const FNetCollectReferencesArgs&);

private:
	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
		virtual void OnPostFreezeNetSerializerRegistry() override;
	};

	static void GameplayTagContainerToSerializationHelper(const SourceType& TagContainer, FGameplayTagContainerNetSerializerSerializationHelper& Out);

	static FGameplayTagContainerNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
	static FStructNetSerializerConfig StructNetSerializerConfig;
	static const FNetSerializer* StructNetSerializer;
};
UE_NET_IMPLEMENT_SERIALIZER(FGameplayTagContainerNetSerializer);

const FGameplayTagContainerNetSerializer::ConfigType FGameplayTagContainerNetSerializer::DefaultConfig;

FGameplayTagContainerNetSerializer::FNetSerializerRegistryDelegates FGameplayTagContainerNetSerializer::NetSerializerRegistryDelegates;
FStructNetSerializerConfig FGameplayTagContainerNetSerializer::StructNetSerializerConfig;
const FNetSerializer* FGameplayTagContainerNetSerializer::StructNetSerializer = &UE_NET_GET_SERIALIZER(FStructNetSerializer);

void FGameplayTagContainerNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	FNetSerializeArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->Serialize(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	FNetDeserializeArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->Deserialize(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::SerializeDelta(FNetSerializationContext& Context, const FNetSerializeDeltaArgs& Args)
{
	FNetSerializeDeltaArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->SerializeDelta(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::DeserializeDelta(FNetSerializationContext& Context, const FNetDeserializeDeltaArgs& Args)
{
	FNetDeserializeDeltaArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->DeserializeDelta(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
	if (const bool bUseFastReplication = TagManager.ShouldUseFastReplication())
	{
		FGameplayTagContainerNetSerializerSerializationHelper IntermediateValue;

		// It's preferred but not vital that the default state represents the true default state. Because of the brittle tag loading/adding we choose to not store tags in the default state.
		if (!Context.IsInitializingDefaultState())
		{
			const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
			GameplayTagContainerToSerializationHelper(SourceValue, IntermediateValue);
		}

		FNetQuantizeArgs InternalArgs = Args;
		InternalArgs.Source = NetSerializerValuePointer(&IntermediateValue);
		InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
		return StructNetSerializer->Quantize(Context, InternalArgs);
	}
	else
	{
		// If we do not use FastReplication we just quantize the tags without sorting them.
		FNetQuantizeArgs InternalArgs = Args;
		InternalArgs.Source = Args.Source;
		InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
		return StructNetSerializer->Quantize(Context, InternalArgs);
	}
}

void FGameplayTagContainerNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	FGameplayTagContainerNetSerializerSerializationHelper IntermediateValue;

	FNetDequantizeArgs InternalArgs = Args;
	InternalArgs.Target = NetSerializerValuePointer(&IntermediateValue);
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;

	StructNetSerializer->Dequantize(Context, InternalArgs);

	FGameplayTagContainer& TargetValue = *reinterpret_cast<FGameplayTagContainer*>(Args.Target);
	TargetValue = FGameplayTagContainer::CreateFromArray(IntermediateValue.GameplayTags);
}

bool FGameplayTagContainerNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
		if (const bool bUseFastReplication = TagManager.ShouldUseFastReplication())
		{
			FNetIsEqualArgs InternalArgs = Args;
			InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
			return StructNetSerializer->IsEqual(Context, InternalArgs);
		}
		else
		{
			// For now as we are rarely using this path and with dynamic tag indices we cannot stable sort them as we do for the fast path
			// so for now we dequantize and compare using the native compare
			SourceType SourceValue0;
			SourceType SourceValue1;

			FNetDequantizeArgs DequantizeArgs = {};
			DequantizeArgs.NetSerializerConfig = Args.NetSerializerConfig;

			DequantizeArgs.Source = Args.Source0;
			DequantizeArgs.Target = NetSerializerValuePointer(&SourceValue0);
			Dequantize(Context, DequantizeArgs);

			DequantizeArgs.Source = Args.Source1;
			DequantizeArgs.Target = NetSerializerValuePointer(&SourceValue1);
			Dequantize(Context, DequantizeArgs);

			if (SourceValue0.GetGameplayTags().Num() != SourceValue1.GetGameplayTags().Num())
			{
				return false;
			}

			if (!SourceValue0.HasAllExact(SourceValue1))
			{
				return false;
			}

			// This will detect duplicate tags.
			if (!SourceValue1.HasAllExact(SourceValue0))
			{
				return false;
			}

			return true;
		}
	}
	else
	{
		const SourceType& SourceValue0 = *reinterpret_cast<const SourceType*>(Args.Source0);
		const SourceType& SourceValue1 = *reinterpret_cast<const SourceType*>(Args.Source1);

		if (SourceValue0.GetGameplayTags().Num() != SourceValue1.GetGameplayTags().Num())
		{
			return false;
		}

		if (!SourceValue0.HasAllExact(SourceValue1))
		{
			return false;
		}

		// This will detect duplicate tags.
		if (!SourceValue1.HasAllExact(SourceValue0))
		{
			return false;
		}

		return true;
	}
}

bool FGameplayTagContainerNetSerializer::Validate(FNetSerializationContext& Context, const FNetValidateArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);

	const int32 NumBitsForContainerSize = UGameplayTagsManager::Get().NumBitsForContainerSize;
	// More than 31 bits would result in undefined behavior for shifting and more than 32 bits is more than the bitstream API supports in a single call.
	if (NumBitsForContainerSize <= 0 || NumBitsForContainerSize > 31)
	{
		// This is worthy of a check as not even a zero sized container can be sent with zero bits.
		checkf(NumBitsForContainerSize > 0, TEXT("Incorrectly configured GameplayTagsManager with %d bits for container size."), NumBitsForContainerSize);
		return false;
	}

	const uint32 NumTags = static_cast<uint32>(SourceValue.GetGameplayTags().Num());
	const uint32 MaxSize = (1U << static_cast<uint32>(NumBitsForContainerSize)) - 1U;
	if (NumTags > MaxSize)
	{
		return false;
	}

	FGameplayTagContainerNetSerializerSerializationHelper IntermediateValue;
	GameplayTagContainerToSerializationHelper(SourceValue, IntermediateValue);

	FNetValidateArgs InternalArgs = Args;
	InternalArgs.Source = NetSerializerValuePointer(&SourceValue.GetGameplayTags());
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;

	return StructNetSerializer->Validate(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::CollectNetReferences(FNetSerializationContext& Context, const FNetCollectReferencesArgs& Args)
{
	// There are no references.
}

void FGameplayTagContainerNetSerializer::CloneDynamicState(FNetSerializationContext& Context, const FNetCloneDynamicStateArgs& Args)
{
	FNetCloneDynamicStateArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->CloneDynamicState(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::FreeDynamicState(FNetSerializationContext& Context, const FNetFreeDynamicStateArgs& Args)
{
	FNetFreeDynamicStateArgs InternalArgs = Args;
	InternalArgs.NetSerializerConfig = &StructNetSerializerConfig;
	return StructNetSerializer->FreeDynamicState(Context, InternalArgs);
}

void FGameplayTagContainerNetSerializer::GameplayTagContainerToSerializationHelper(const SourceType& TagContainer, FGameplayTagContainerNetSerializerSerializationHelper& Out)
{
	const UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
	const int32 TagCount = TagContainer.GetGameplayTags().Num();
	if (TagCount <= 0)
	{
		return;
	}

	struct FPair
	{
		SIZE_T ArrayIndex;
		FGameplayTagNetIndex TagIndex;
	};

	// Needs sorted array in order to determine equality between quantized states. Can also be used to implement more bandwidth efficient serialization.
	TArray<FPair> SortedArray;
	SortedArray.AddZeroed(TagCount);
	{
		FPair* IndexPairs = SortedArray.GetData();
		SIZE_T IndexPairCount = 0;
		for (const FGameplayTag& Tag : TagContainer)
		{
			IndexPairs[IndexPairCount].ArrayIndex = IndexPairCount;
			IndexPairs[IndexPairCount].TagIndex = TagManager.GetNetIndexFromTag(Tag);
			++IndexPairCount;
		}

		auto&& SortByTagIndex = [](const FPair& Value0, const FPair& Value1)->bool
		{
			if (Value0.TagIndex != Value1.TagIndex)
			{
				return Value0.TagIndex < Value1.TagIndex;
			}
			else
			{
				return Value0.ArrayIndex < Value1.ArrayIndex;
			}
		};
		Algo::Sort(SortedArray, SortByTagIndex);
	}

	// Copy sorted tags to helper struct instance.
	{
		Out.GameplayTags.SetNum(TagCount);
		FGameplayTag* TargetTags = Out.GameplayTags.GetData();
		const FGameplayTag* SourceTags = TagContainer.GetGameplayTags().GetData();
		SIZE_T TargetIndex = 0;
		for (const FPair& Pair : SortedArray)
		{
			TargetTags[TargetIndex] = SourceTags[Pair.ArrayIndex];
			++TargetIndex;
		}
	}
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayTagContainer("GameplayTagContainer");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTagContainer, FGameplayTagContainerNetSerializer);

FGameplayTagContainerNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTagContainer);
}

void FGameplayTagContainerNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTagContainer);
}

void FGameplayTagContainerNetSerializer::FNetSerializerRegistryDelegates::OnPostFreezeNetSerializerRegistry()
{
	UStruct* Struct = FGameplayTagContainerNetSerializerSerializationHelper::StaticStruct();
	StructNetSerializerConfig.StateDescriptor = FReplicationStateDescriptorBuilder::CreateDescriptorForStruct(Struct);
	const FReplicationStateDescriptor* Descriptor = StructNetSerializerConfig.StateDescriptor.GetReference();

	check(Descriptor != nullptr);
	// We have an empty CollectNetReferences implementation so make sure everything is as expected.
	check(!EnumHasAnyFlags(Descriptor->Traits, EReplicationStateTraits::HasObjectReference));

	// Validate our assumptions regarding quantized state size and alignment.
	static_assert(offsetof(FQuantizedType, QuantizedStruct) == 0U, "Expected buffer for struct to be first member of FQuantizedType.");
	if (sizeof(FQuantizedType::QuantizedStruct) < Descriptor->InternalSize || alignof(FQuantizedType) < Descriptor->InternalAlignment)
	{
		LowLevelFatalError(TEXT("FQuantizedType::QuantizedStruct has size %u and alignment %u but requires size %u and alignment %u."), uint32(sizeof(FQuantizedType::QuantizedStruct)), uint32(alignof(FQuantizedType)), uint32(Descriptor->InternalSize), uint32(Descriptor->InternalAlignment)); 
	}
}

}

