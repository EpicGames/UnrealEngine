// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagNetSerializer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagNetSerializer)


#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "GameplayTagTokenStore.h"
#include "Iris/ReplicationState/PropertyNetSerializerInfoRegistry.h"
#include "Iris/Serialization/NetBitStreamUtil.h"
#include "Iris/Serialization/NetSerializerDelegates.h"
#include "Iris/ReplicationSystem/ReplicationSystem.h"

static_assert(sizeof(FGameplayTagNetIndex) == 2, "Unexpected GameplayTagNetIndex size. Expected 2.");

namespace UE::Net
{

struct FGameplayTagAccessorForNetSerializer : public FGameplayTag
{
	void SetTagName(const FName InTagName) { TagName = InTagName; }
};

// Types
struct FFGameplayTagNetSerializerQuantizedType
{
	union
	{
		FNetToken TagNetToken;		
		FGameplayTagNetIndex TagIndex;
	};
	bool bUseFastReplication;
};

}

template <> struct TIsPODType<UE::Net::FFGameplayTagNetSerializerQuantizedType> { enum { Value = true }; };

namespace UE::Net
{

struct FGameplayTagNetSerializer
{
	// Version
	static const uint32 Version = 0;

	typedef FGameplayTag SourceType;
	typedef FFGameplayTagNetSerializerQuantizedType QuantizedType;
	typedef struct FGameplayTagNetSerializerConfig ConfigType;

	static const ConfigType DefaultConfig;

	//
	static void Serialize(FNetSerializationContext&, const FNetSerializeArgs&);
	static void Deserialize(FNetSerializationContext&, const FNetDeserializeArgs&);

	static void Quantize(FNetSerializationContext&, const FNetQuantizeArgs&);
	static void Dequantize(FNetSerializationContext&, const FNetDequantizeArgs&);

	static bool IsEqual(FNetSerializationContext&, const FNetIsEqualArgs&);

private:
	static constexpr FGameplayTagNetIndex InvalidTagIndex = INVALID_TAGNETINDEX;

	class FNetSerializerRegistryDelegates final : private UE::Net::FNetSerializerRegistryDelegates
	{
	public:
		virtual ~FNetSerializerRegistryDelegates();

	private:
		virtual void OnPreFreezeNetSerializerRegistry() override;
	};

	static FGameplayTagNetSerializer::FNetSerializerRegistryDelegates NetSerializerRegistryDelegates;
};
UE_NET_IMPLEMENT_SERIALIZER(FGameplayTagNetSerializer);

const FGameplayTagNetSerializer::ConfigType FGameplayTagNetSerializer::DefaultConfig;
FGameplayTagNetSerializer::FNetSerializerRegistryDelegates FGameplayTagNetSerializer::NetSerializerRegistryDelegates;

void FGameplayTagNetSerializer::Serialize(FNetSerializationContext& Context, const FNetSerializeArgs& Args)
{
	const QuantizedType& Value = *reinterpret_cast<QuantizedType*>(Args.Source);
	
	FNetBitStreamWriter* Writer = Context.GetBitStreamWriter();
	
	if (Writer->WriteBool(Value.bUseFastReplication))
	{
		WritePackedUint16(Writer, Value.TagIndex);
	}
	else
	{
		// Tokens will differ, so we cannot store them in the default statehash.
		if (Context.IsInitializingDefaultState())
		{
			return;
		}

		// Write token without type, 
		Context.GetNetTokenStore()->WriteNetTokenWithKnownType<FGameplayTagTokenStore>(Context, Value.TagNetToken);

		// Export or add to pending exports for later export
		FNetTokenStore::AppendExport(Context, Value.TagNetToken);
	}
}

void FGameplayTagNetSerializer::Deserialize(FNetSerializationContext& Context, const FNetDeserializeArgs& Args)
{
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);
	
	FNetBitStreamReader* Reader = Context.GetBitStreamReader();

	TargetValue = {};

	if (const bool bUseFastReplication = Reader->ReadBool())
	{
		TargetValue.TagIndex = ReadPackedUint16(Reader);
		TargetValue.bUseFastReplication = true;
	}
	else
	{
		FNetToken NetToken = Context.GetNetTokenStore()->ReadNetTokenWithKnownType<FGameplayTagTokenStore>(Context);
		if (Reader->IsOverflown())
		{
			return;
		}

		if (Reader->IsOverflown())
		{
			return;
		}

		// Store 
		TargetValue.TagNetToken = NetToken;
	}
}

void FGameplayTagNetSerializer::Quantize(FNetSerializationContext& Context, const FNetQuantizeArgs& Args)
{
	const SourceType& SourceValue = *reinterpret_cast<const SourceType*>(Args.Source);
	QuantizedType& TargetValue = *reinterpret_cast<QuantizedType*>(Args.Target);

	TargetValue = {};

	const UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();

	const bool bUseFastReplication = TagManager.ShouldUseFastReplication();
	
	if (bUseFastReplication)
	{
		TargetValue.bUseFastReplication = true;
		
		// We use a stable value for invalid TagIndex as the value from the TagManager is dynamic.
		FGameplayTagNetIndex TagIndex = TagManager.GetNetIndexFromTag(SourceValue);
		if (TagIndex == TagManager.GetInvalidTagNetIndex())
		{
			TargetValue.TagIndex = InvalidTagIndex;
		}
		else
		{
			TargetValue.TagIndex = TagIndex;
		}
	}
	else
	{
		if (FGameplayTagTokenStore* TagTokenStore = Context.GetNetTokenStore()->GetDataStore<FGameplayTagTokenStore>())
		{
			TargetValue.TagNetToken = TagTokenStore->GetOrCreateToken(SourceValue);
		}
		else
		{
			UE_LOGF(LogGameplayTags, Error, "FGameplayTagNetSerializer::Quantize Could not find required FGameplayTagTokenStore");
			ensure(false);
		}
	}
}

void FGameplayTagNetSerializer::Dequantize(FNetSerializationContext& Context, const FNetDequantizeArgs& Args)
{
	const QuantizedType& Source = *reinterpret_cast<const QuantizedType*>(Args.Source);
	SourceType& Target = *reinterpret_cast<SourceType*>(Args.Target);

	if (Source.bUseFastReplication)
	{
		if (Source.TagIndex != InvalidTagIndex)
		{
			FGameplayTagAccessorForNetSerializer& TargetAccessor = *reinterpret_cast<FGameplayTagAccessorForNetSerializer*>(Args.Target);
			const UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
			TargetAccessor.SetTagName(TagManager.GetTagNameFromNetIndex(Source.TagIndex));
		}
		else
		{
			// Invalid Tag
			Target = FGameplayTag();
		}
	}
	else
	{
		if (FGameplayTagTokenStore* TagTokenStore = Context.GetNetTokenStore()->GetDataStore<FGameplayTagTokenStore>())
		{
			Target = TagTokenStore->ResolveToken(Source.TagNetToken, Context.GetRemoteNetTokenStoreState());
		}
		else
		{
			UE_LOGF(LogGameplayTags, Error, "FGameplayTagNetSerializer::Dequantize Could not find required FGameplayTagTokenStore");
			ensure(false);

			// Invalid Tag
			Target = FGameplayTag();

			return;
		}
	}
}

bool FGameplayTagNetSerializer::IsEqual(FNetSerializationContext& Context, const FNetIsEqualArgs& Args)
{
	if (Args.bStateIsQuantized)
	{
		const QuantizedType& Value0 = *reinterpret_cast<const QuantizedType*>(Args.Source0);
		const QuantizedType& Value1 = *reinterpret_cast<const QuantizedType*>(Args.Source1);

		if (Value0.bUseFastReplication != Value1.bUseFastReplication)
		{
			return false;
		}

		if (Value0.bUseFastReplication)
		{
			return Value0.TagIndex == Value1.TagIndex;
		}
		else
		{
			// Need to compare actual Tags to properly compare non-auth and auth token
			if (Value0.TagNetToken.IsAssignedByAuthority() != Value1.TagNetToken.IsAssignedByAuthority())
			{
				FGameplayTagTokenStore* TagTokenStore = Context.GetNetTokenStore()->GetDataStore<FGameplayTagTokenStore>();
				const UE::Net::FNetTokenStoreState* RemoteNetTokenStoreState = Context.GetRemoteNetTokenStoreState();
	
				const FGameplayTag Tag0 = TagTokenStore->ResolveToken(Value0.TagNetToken, RemoteNetTokenStoreState);
				const FGameplayTag Tag1 = TagTokenStore->ResolveToken(Value1.TagNetToken, RemoteNetTokenStoreState);
				
				if (Tag0 != Tag1)
				{
					return false;
				}
			}
			else if (Value0.TagNetToken != Value1.TagNetToken)
			{
				return false;
			}
			return true;
		}
	}
	else
	{
		const SourceType& Value0 = *reinterpret_cast<SourceType*>(Args.Source0);
		const SourceType& Value1 = *reinterpret_cast<SourceType*>(Args.Source1);

		return Value0.GetTagName() == Value1.GetTagName();
	}
}

static const FName PropertyNetSerializerRegistry_NAME_GameplayTag("GameplayTag");
UE_NET_IMPLEMENT_NAMED_STRUCT_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTag, FGameplayTagNetSerializer);

FGameplayTagNetSerializer::FNetSerializerRegistryDelegates::~FNetSerializerRegistryDelegates()
{
	UE_NET_UNREGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTag);
}

void FGameplayTagNetSerializer::FNetSerializerRegistryDelegates::OnPreFreezeNetSerializerRegistry()
{
	UE_NET_REGISTER_NETSERIALIZER_INFO(PropertyNetSerializerRegistry_NAME_GameplayTag);
}

}

