// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagRedirectors.h"
#include "GameplayTagsSettings.h"
#include "GameplayTagsManager.h"
#include "Misc/ConfigCacheIni.h"
#if WITH_EDITOR
#include "Algo/Sort.h"
#include "Hash/Blake3.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagRedirectors)

FGameplayTagRedirectors& FGameplayTagRedirectors::Get()
{
	static FGameplayTagRedirectors Singleton;
	return Singleton;
}

FGameplayTagRedirectors::FGameplayTagRedirectors()
{
	UGameplayTagsSettings* MutableDefault = GetMutableDefault<UGameplayTagsSettings>();

	// Check the deprecated location
	bool bFoundDeprecated = false;
	const FConfigSection* PackageRedirects = GConfig->GetSection(TEXT("/Script/Engine.Engine"), false, GEngineIni);

	if (PackageRedirects)
	{
		for (FConfigSection::TConstIterator It(*PackageRedirects); It; ++It)
		{
			if (It.Key() == TEXT("+GameplayTagRedirects"))
			{
				FName OldTagName = NAME_None;
				FName NewTagName;

				if (FParse::Value(*It.Value().GetValue(), TEXT("OldTagName="), OldTagName))
				{
					if (FParse::Value(*It.Value().GetValue(), TEXT("NewTagName="), NewTagName))
					{
						FGameplayTagRedirect Redirect;
						Redirect.OldTagName = OldTagName;
						Redirect.NewTagName = NewTagName;

						MutableDefault->GameplayTagRedirects.AddUnique(Redirect);

						bFoundDeprecated = true;
					}
				}
			}
		}
	}

	if (bFoundDeprecated)
	{
		UE_LOGF(LogGameplayTags, Error, "GameplayTagRedirects is in a deprecated location, after editing GameplayTags developer settings you must remove these manually");
	}

#if WITH_EDITOR
	// Only doing the deprecated parse once at startup is fine, but we need to update from the settings object after in-editor config updates
	// This is a singleton that is never destroyed, so just bind raw once
	UGameplayTagsManager::OnEditorRefreshGameplayTagTree.AddRaw(this, &FGameplayTagRedirectors::RefreshTagRedirects);
#endif

	RefreshTagRedirects();
}

void FGameplayTagRedirectors::AddRedirectsFromSource(const FGameplayTagSource* Source)
{
	if (Source && Source->SourceTagList)
	{
		AddRedirects(Source->SourceTagList->GameplayTagRedirects);
	}
}

void FGameplayTagRedirectors::RefreshTagRedirects()
{
	TagRedirects.Empty();

	// Check settings object
	const UGameplayTagsSettings* Default = GetDefault<UGameplayTagsSettings>();
	AddRedirects(Default->GameplayTagRedirects);

	// Add redirects from all TagList sources
	UGameplayTagsManager& GameplayTagsManager = UGameplayTagsManager::Get();
	TArray<FGameplayTagRedirect> RedirectsFromTagSources;
	TArray<const FGameplayTagSource*> TagListSources;
	GameplayTagsManager.FindTagSourcesWithType(EGameplayTagSourceType::TagList, TagListSources);
	for (const FGameplayTagSource* Source : TagListSources)
	{
		AddRedirectsFromSource(Source);
	}

}

bool FGameplayTagRedirectors::RedirectTag(const FName& InTagName, FGameplayTag& OutTag) const
{
	if (const FGameplayTag* NewTag = TagRedirects.Find(InTagName))
	{
		OutTag = *NewTag;
		return true;
	}

	return false;
}

void FGameplayTagRedirectors::AddRedirects(const TArray<FGameplayTagRedirect>& Redirects)
{
	for (const FGameplayTagRedirect& Redirect : Redirects)
	{
		FName OldTagName = Redirect.OldTagName;
		FName NewTagName = Redirect.NewTagName;

		const FGameplayTag* ExistingRedirect = TagRedirects.Find(OldTagName);
		if (!ExistingRedirect)
		{
			// Attempt to find multiple redirect hops and flatten the redirection so we only need to redirect once
			// to resolve the update.  Includes a basic infinite recursion guard, in case the redirects loop.
			int32 IterationsLeft = 10;
			while (NewTagName != NAME_None)
			{
				bool bFoundRedirect = false;

				// See if it got redirected again
				for (const FGameplayTagRedirect& SecondRedirect : Redirects)
				{
					if (SecondRedirect.OldTagName == NewTagName)
					{
						NewTagName = SecondRedirect.NewTagName;
						bFoundRedirect = true;
						break;
					}
				}
				IterationsLeft--;

				if (!bFoundRedirect)
				{
					break;
				}

				if (IterationsLeft <= 0)
				{
					UE_LOGF(LogGameplayTags, Warning, "Invalid new tag %ls!  Cannot replace old tag %ls.", *Redirect.NewTagName.ToString(), *Redirect.OldTagName.ToString());
					break;
				}
			}

			// Populate the map
			TagRedirects.Add(OldTagName, FGameplayTag(NewTagName));
		}
		else
		{
			ensureMsgf(ExistingRedirect->GetTagName() == NewTagName, TEXT("Old tag %s is being redirected to more than one tag. Please remove all the redirections except for one. NewTagName:%s ExistingRedirect:%s"), *OldTagName.ToString(), *NewTagName.ToString(), *ExistingRedirect->GetTagName().ToString());
		}
	}
}

#if WITH_EDITOR
void FGameplayTagRedirectors::Hash(FBlake3& Hasher)
{
	TArray<FName> SortedKeys;
	TagRedirects.GenerateKeyArray(SortedKeys);
	Algo::Sort(SortedKeys, FNameLexicalLess());

	FNameBuilder Builder;
	for (FName Key : SortedKeys)
	{
		FGameplayTag* Value = TagRedirects.Find(Key);
		check(Value);
		Builder.Reset();
		Builder << Key;
		for (TCHAR& C : Builder)
		{
			C = FChar::ToLower(C);
		}
		Hasher.Update(*Builder, Builder.Len() * sizeof(**Builder));
		Builder.Reset();
		Builder << Value->GetTagName();
		for (TCHAR& C : Builder)
		{
			C = FChar::ToLower(C);
		}
		Hasher.Update(*Builder, Builder.Len() * sizeof(**Builder));
	}
}

bool FGameplayTagRedirectors::FindRedirectorsFromTargetTagName(const FName& TargetTagName, TArray<FName>& SourceTagNames) const
{
	for (const TPair<FName, FGameplayTag>& It : TagRedirects) // we should have a reverse map instead of looping
	{
		if (It.Value.GetTagName() == TargetTagName)
		{
			SourceTagNames.Add(It.Key);
		}
	}

	return !SourceTagNames.IsEmpty();
}
#endif
