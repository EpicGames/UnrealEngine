// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagTokenStore.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"

namespace UE::Net
{

 FGameplayTagTokenStore::FGameplayTagTokenStore(FNetTokenStore& TokenStore) : FNameTokenStore(TokenStore)
 {
 }

FNetToken FGameplayTagTokenStore::GetOrCreateToken(FGameplayTag Tag)
{ 
	return FNameTokenStore::GetOrCreateToken(Tag.GetTagName());
}

FGameplayTag FGameplayTagTokenStore::ResolveToken(FNetToken Token, const FNetTokenStoreState* RemoteTokenStoreState) const
{
	const FName TagName = FNameTokenStore::ResolveToken(Token, RemoteTokenStoreState);
	return TagName.IsNone() ? FGameplayTag() : UGameplayTagsManager::Get().RequestGameplayTag(TagName);
}

}
