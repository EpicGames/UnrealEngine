// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsManager.h"
#include "Containers/AllowShrinking.h"
#include "Containers/Array.h"
#include "Engine/Engine.h"
#include "GameplayTagContainer.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/FileManager.h"
#include "Logging/LogMacros.h"
#include "Logging/StructuredLog.h"
#include "Misc/Paths.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Stats/StatsMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/UObjectArray.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "UObject/LinkerLoad.h"
#include "UObject/Package.h"
#include "UObject/UObjectThreadContext.h"
#include "GameplayTagsSettings.h"
#include "GameplayTagsModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Misc/AsciiSet.h"
#include "Misc/CoreDelegates.h"
#include "Misc/FileHelper.h"
#include "Misc/CommandLine.h"
#include "HAL/IConsoleManager.h"
#include "NativeGameplayTags.h"
#include "AutoRTFM.h"
#include "String/Join.h"

#if WITH_EDITOR
#include "SourceControlHelpers.h"
#include "ISourceControlModule.h"
#include "Algo/Sort.h"
#include "Cooker/CookDependency.h"
#include "Cooker/CookDependencyContext.h"
#include "UObject/ICookInfo.h"
#include "Editor.h"
#include "PropertyHandle.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Serialization/CompactBinarySerialization.h"
FSimpleMulticastDelegate UGameplayTagsManager::OnEditorRefreshGameplayTagTree;
#endif

#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Containers/Queue.h"
#include "Templates/Function.h"
#include "Templates/Greater.h"
#include "UObject/StrongObjectPtr.h"
#include "Async/Async.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsManager)

const FName UGameplayTagsManager::NAME_Categories("Categories");
const FName UGameplayTagsManager::NAME_GameplayTagFilter("GameplayTagFilter");

#define LOCTEXT_NAMESPACE "GameplayTagManager"

DECLARE_CYCLE_STAT(TEXT("Load Gameplay Tags"), STAT_GameplayTags_LoadGameplayTags, STATGROUP_GameplayTags);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Add Tag *.ini Search Path"), STAT_GameplayTags_AddTagIniSearchPath, STATGROUP_GameplayTags);
DECLARE_FLOAT_ACCUMULATOR_STAT(TEXT("Remove Tag *.ini Search Path"), STAT_GameplayTags_RemoveTagIniSearchPath, STATGROUP_GameplayTags);

#if !(UE_BUILD_SHIPPING)

static FAutoConsoleCommand PrintReplicationIndicesCommand(
	TEXT("GameplayTags.PrintReplicationIndicies"),
	TEXT("Prints the index assigned to each tag for fast network replication."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		UGameplayTagsManager::Get().PrintReplicationIndices();
	})
);

#endif

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

static FAutoConsoleCommand PrintReplicationFrequencyReportCommand(
	TEXT("GameplayTags.PrintReplicationFrequencyReport"),
	TEXT("Prints the frequency each tag is replicated."),
	FConsoleCommandDelegate::CreateLambda([]()
	{
		UGameplayTagsManager::Get().PrintReplicationFrequencyReport();
	})
);

#endif

#if WITH_EDITOR
static FAutoConsoleCommand CMD_DumpGameplayTagSources(
	TEXT("GameplayTags.DumpSources"),
	TEXT("Dumps all known sources of gameplay tags"),
	FConsoleCommandWithOutputDeviceDelegate::CreateLambda([](FOutputDevice& Out){
		UGameplayTagsManager::Get().DumpSources(Out);
	})
);
#endif

static TAutoConsoleVariable<int32> CVarGameplayTagsLruCacheSize(
	TEXT("GameplayTags.LruCacheSize"),
	3,
	TEXT("Accelerate consequent RequestGameplayTagChildren searches at the cost of the memory usage.")
);

struct FCompareFGameplayTagNodeByTag
{
	FORCEINLINE bool operator()(const TSharedPtr<FGameplayTagNode>& A, const TSharedPtr<FGameplayTagNode>& B) const
	{
		// Note: GetSimpleTagName() is not good enough here. The individual tag nodes are share frequently (E.g, Dog.Tail, Cat.Tail have sub nodes with the same simple tag name)
		// Compare with equal FNames will look at the backing number/indices to the FName. For FNames used elsewhere, like "A" for example, this can cause non determinism in platforms
		// (For example if static order initialization differs on two platforms, the "version" of the "A" FName that two places get could be different, causing this comparison to also be)
		return (A->GetCompleteTagName().Compare(B->GetCompleteTagName())) < 0;
	}
};

namespace GameplayTagUtil
{
	static void GetRestrictedConfigsFromIni(const FString& IniFilePath, TArray<FRestrictedConfigInfo>& OutRestrictedConfigs)
	{
		FConfigFile ConfigFile;
		ConfigFile.Read(IniFilePath);

		TArray<FString> IniConfigStrings;
		if (ConfigFile.GetArray(TEXT("/Script/GameplayTags.GameplayTagsSettings"), TEXT("RestrictedConfigFiles"), IniConfigStrings))
		{
			for (const FString& ConfigString : IniConfigStrings)
			{
				FRestrictedConfigInfo Config;
				if (FRestrictedConfigInfo::StaticStruct()->ImportText(*ConfigString, &Config, nullptr, PPF_None, nullptr, FRestrictedConfigInfo::StaticStruct()->GetName()))
				{
					OutRestrictedConfigs.Add(Config);
				}
			}
		}
	}

#if !UE_BUILD_SHIPPING
	static void GatherGameplayTagStringsRecursive(const FGameplayTagNode& RootNode, TArray<FString>& Out)
	{
		Out.Add(RootNode.GetCompleteTagString());

		for (const TSharedPtr<FGameplayTagNode>& ChildNode : RootNode.GetChildTagNodes())
		{
			GatherGameplayTagStringsRecursive(*ChildNode, Out);
		}
	}

	static void DumpGameplayTagStrings(const FGameplayTagNode& RootNode, const FString& Filename)
	{
		TArray<FString> Lines;
		GatherGameplayTagStringsRecursive(RootNode, Lines);
		Lines.Sort();

		FString TagDumpFilename = FPaths::ProjectSavedDir() / Filename;
		if (!FFileHelper::SaveStringArrayToFile(Lines, *TagDumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
		{
			UE_LOGF(LogGameplayTags, Display, "Wrote Tag Dump: %ls", *TagDumpFilename);
		}
	}	

	static void DumpRegisteredSearchPaths(const TMap<FString, FGameplayTagSearchPathInfo>& RegisteredSearchPaths, const FString& Filename)
	{
		TArray<FString> Lines;
		for (const TPair<FString, FGameplayTagSearchPathInfo>& Pair : RegisteredSearchPaths)
		{
			Lines.Add(FString::Printf(TEXT("%s bWasSearched:%d bWasAddedToTree:%d"), *Pair.Key, int32(Pair.Value.bWasSearched), int32(Pair.Value.bWasAddedToTree)));

			for (int32 Idx = 0; Idx < Pair.Value.SourcesInPath.Num(); ++Idx)
			{
				Lines.Add(FString::Printf(TEXT("%s SourcesInPath[%d]: %s"), *Pair.Key, Idx, *Pair.Value.SourcesInPath[Idx].ToString()));
			}

			for (int32 Idx = 0; Idx < Pair.Value.TagIniList.Num(); ++Idx)
			{
				Lines.Add(FString::Printf(TEXT("%s TagIniList[%d]: %s"), *Pair.Key, Idx, *Pair.Value.TagIniList[Idx]));
			}
		}

		Lines.Sort();

		FString DumpFilename = FPaths::ProjectSavedDir() / Filename;
		if (!FFileHelper::SaveStringArrayToFile(Lines, *DumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
		{
			UE_LOGF(LogGameplayTags, Display, "Wrote RegisteredSearchPaths Dump: %ls", *DumpFilename);
		}
	}

	static void DumpRestrictedGameplayTagSourceNames(const TSet<FName>& RestrictedGameplayTagSourceNames, const FString& Filename)
	{
		TArray<FString> Lines;

		for (const FName Name : RestrictedGameplayTagSourceNames)
		{
			Lines.Add(Name.ToString());
		}

		Lines.Sort();

		FString DumpFilename = FPaths::ProjectSavedDir() / Filename;
		if (!FFileHelper::SaveStringArrayToFile(Lines, *DumpFilename, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM))
		{
			UE_LOGF(LogGameplayTags, Display, "Wrote RestrictedGameplayTagSourceNames Dump: %ls", *DumpFilename);
		}
	}
#endif

#if WITH_EDITOR
	struct FGameplayTagCookDependency
	{
	public:
		FGameplayTagCookDependency()
			: TagName()
			, SourceTagNames()
			, bTagExists(false)
		{ }

		void Save(FCbWriter& Writer)
		{
			Writer << TagName;

			SourceTagNames.Sort(FNameLexicalLess());
			Writer << SourceTagNames;
			
			Writer << bTagExists;
		}

		bool Load(FCbFieldViewIterator& Args)
		{
			FUtf8StringView TagNameAsString = Args.AsString();
			TagName = FName(TagNameAsString);

			bool bSuccess = LoadFromCompactBinary((++Args), SourceTagNames);
			if (!bSuccess)
			{
				return false;
			}

			bTagExists = (++Args).AsBool();

			return true;
		}

		void Hash(FBlake3& OutHasher)
		{
			FString TagNameString = TagName.ToString();
			TagNameString.ToLowerInline();
			OutHasher.Update(TagNameString.GetCharArray().GetData(), TagNameString.NumBytesWithoutNull());

			SourceTagNames.Sort(FNameLexicalLess());
			for (const FName& SourceTagName : SourceTagNames)
			{
				FString SourceTagNameString = SourceTagName.ToString();
				SourceTagNameString.ToLowerInline();
				OutHasher.Update(SourceTagNameString.GetCharArray().GetData(), SourceTagNameString.NumBytesWithoutNull());
			}

			OutHasher.Update(&bTagExists, sizeof(bTagExists));
		}

		FName TagName;
		TArray<FName> SourceTagNames; //if the tag was redirected, contains all the source tag names.
		bool bTagExists;
	};
#endif //#if WITH_EDITOR
}


//////////////////////////////////////////////////////////////////////
// FGameplayTagSource

static const FName NAME_Native = FName("Native");
static const FName NAME_DefaultGameplayTagsIni("DefaultGameplayTags.ini");

FString FGameplayTagSource::GetConfigFileName() const
{
	if (SourceTagList)
	{
		return SourceTagList->ConfigFileName;
	}
	if (SourceRestrictedTagList)
	{
		return SourceRestrictedTagList->ConfigFileName;
	}

	return FString();
}

FName FGameplayTagSource::GetNativeName()
{
	return NAME_Native;
}

FName FGameplayTagSource::GetDefaultName()
{
	return NAME_DefaultGameplayTagsIni;
}

#if WITH_EDITOR
static const FName NAME_TransientEditor("TransientEditor");

FName FGameplayTagSource::GetFavoriteName()
{
	return GetDefault<UGameplayTagsDeveloperSettings>()->FavoriteTagSource;
}

void FGameplayTagSource::SetFavoriteName(FName TagSourceToFavorite)
{
	UGameplayTagsDeveloperSettings* MutableSettings = GetMutableDefault<UGameplayTagsDeveloperSettings>();

	if (MutableSettings->FavoriteTagSource != TagSourceToFavorite)
	{
		MutableSettings->Modify();
		MutableSettings->FavoriteTagSource = TagSourceToFavorite;

		FPropertyChangedEvent ChangeEvent(MutableSettings->GetClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UGameplayTagsDeveloperSettings, FavoriteTagSource)), EPropertyChangeType::ValueSet);
		MutableSettings->PostEditChangeProperty(ChangeEvent);
		
		MutableSettings->SaveConfig();
	}
}

FName FGameplayTagSource::GetTransientEditorName()
{
	return NAME_TransientEditor;
}
#endif

//////////////////////////////////////////////////////////////////////
// UGameplayTagsManager

UGameplayTagsManager* UGameplayTagsManager::SingletonManager = nullptr;

UGameplayTagsManager::UGameplayTagsManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bUseFastReplication = false;
	bShouldWarnOnInvalidTags = true;
	bDoneAddingNativeTags = false;
	bShouldAllowUnloadingTags = false;
	NetIndexFirstBitSegment = 16;
	NetIndexTrueBitNum = 16;
	NumBitsForContainerSize = 6;
	NetworkGameplayTagNodeIndexHash = 0;
}

// Enable to turn on detailed startup logging
#define GAMEPLAYTAGS_VERBOSE 0

#if STATS && GAMEPLAYTAGS_VERBOSE
#define SCOPE_LOG_GAMEPLAYTAGS(Name) SCOPE_LOG_TIME_IN_SECONDS(Name, nullptr)
#else
#define SCOPE_LOG_GAMEPLAYTAGS(Name)
#endif

void UGameplayTagsManager::LoadGameplayTagTables(bool bAllowAsyncLoad)
{
	SCOPE_CYCLE_COUNTER(STAT_GameplayTags_LoadGameplayTags);

	const UGameplayTagsSettings* Default = GetDefault<UGameplayTagsSettings>();
	check(Default);

	GameplayTagTables.Empty();

	// Because we may unload the GameplayTags ini file later, foce the construction of other CDOs which use that config file now so that they don't reload it later
	(void)GetDefault<URestrictedGameplayTagsList>();

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST && DO_ENSURE
	// Look for any other classes with the gameplay tags ini file that will cause it to be reloaded later
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		ensureMsgf(ClassIt->ClassConfigName != Default->GetClass()->ClassConfigName
			|| *ClassIt == UGameplayTagsSettings::StaticClass() 
			|| *ClassIt == URestrictedGameplayTagsList::StaticClass(), 
			TEXT("Class %s is using the GameplayTags config file which UGameplayTagsManager will attempt to unload after building the gameplay tag tree. ")
			TEXT("This may cause the GameplayTags config file to be reloaded, consuming additional memory."), *ClassIt->GetPathName() );
	}
#endif // !UE_BUILD_SHIPPING && !UE_BUILD_TEST && DO_ENSURE

#if !WITH_EDITOR
	// If we're a cooked build and in a safe spot, start an async load so we can pipeline it
	if (bAllowAsyncLoad && !IsLoading() && Default->GameplayTagTableList.Num() > 0)
	{
		for (FSoftObjectPath DataTablePath : Default->GameplayTagTableList)
		{
			LoadPackageAsync(DataTablePath.GetLongPackageName());
		}

		return;
	}
#endif // !WITH_EDITOR

	SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::LoadGameplayTagTables"));
	for (FSoftObjectPath DataTablePath : Default->GameplayTagTableList)
	{
		UDataTable* TagTable = LoadObject<UDataTable>(nullptr, *DataTablePath.ToString(), nullptr, LOAD_None, nullptr);

		// Handle case where the module is dynamically-loaded within a LoadPackage stack, which would otherwise
		// result in the tag table not having its RowStruct serialized in time. Without the RowStruct, the tags manager
		// will not be initialized correctly.
		if (TagTable)
		{
			TagTable->ConditionalPreload();
		}
		GameplayTagTables.Add(TagTable);
	}
}

void UGameplayTagsManager::AddTagIniSearchPath(const FString& RootDir, const TSet<FString>* PluginConfigsCache)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameplayTagsManager::AddTagIniSearchPath);
	SCOPE_SECONDS_ACCUMULATOR(STAT_GameplayTags_AddTagIniSearchPath);

	FGameplayTagSearchPathInfo* PathInfo = RegisteredSearchPaths.Find(RootDir);

	if (!PathInfo)
	{
		PathInfo = &RegisteredSearchPaths.FindOrAdd(RootDir);
	}

	if (!PathInfo->bWasSearched)
	{
		PathInfo->Reset();
		
		// Read all tags from the ini
		// Use slower path and check the filesystem if our PluginConfigsCache is null
		if (PluginConfigsCache == nullptr)
		{
			TArray<FString> FilesInDirectory;
			IFileManager::Get().FindFilesRecursive(FilesInDirectory, *RootDir, TEXT("*.ini"), true, false);

			if (FilesInDirectory.Num() > 0)
			{
				FilesInDirectory.Sort();

				for (const FString& IniFilePath : FilesInDirectory)
				{
					const FName TagSource = FName(*FPaths::GetCleanFilename(IniFilePath));
					PathInfo->SourcesInPath.Add(TagSource);
					PathInfo->TagIniList.Add(FConfigCacheIni::NormalizeConfigIniPath(IniFilePath));
				}
			}
		}
		else
		{
			for (const FString& IniFilePath : *PluginConfigsCache)
			{
				// Only grab ini files that part of the root dir we are looking for
				if (IniFilePath.StartsWith(RootDir))
				{
					const FName TagSource = FName(*FPaths::GetCleanFilename(IniFilePath));
					PathInfo->SourcesInPath.Add(TagSource);
					PathInfo->TagIniList.Add(FConfigCacheIni::NormalizeConfigIniPath(IniFilePath));
				}
			}
		}
		PathInfo->bWasSearched = true;
	}

	if (!PathInfo->bWasAddedToTree)
	{
		for (const FString& IniFilePath : PathInfo->TagIniList)
		{
			TArray<FRestrictedConfigInfo> IniRestrictedConfigs;
			GameplayTagUtil::GetRestrictedConfigsFromIni(IniFilePath, IniRestrictedConfigs);
			const FString IniDirectory = FPaths::GetPath(IniFilePath);
			for (const FRestrictedConfigInfo& Config : IniRestrictedConfigs)
			{
				const FString RestrictedFileName = FString::Printf(TEXT("%s/%s"), *IniDirectory, *Config.RestrictedConfigName);
				AddRestrictedGameplayTagSource(RestrictedFileName);
			}
		}

		AddTagsFromAdditionalLooseIniFiles(PathInfo->TagIniList);

		PathInfo->bWasAddedToTree = true;

		HandleGameplayTagTreeChanged(false);
	}
}

bool UGameplayTagsManager::RemoveTagIniSearchPath(const FString& RootDir)
{
	SCOPE_SECONDS_ACCUMULATOR(STAT_GameplayTags_RemoveTagIniSearchPath);

	if (!ShouldUnloadTags())
	{
		// Can't unload at all
		return false;
	}

	FGameplayTagSearchPathInfo* PathInfo = RegisteredSearchPaths.Find(RootDir);

	if (PathInfo)
	{
		// Clear out the path and then recreate the tree
		RegisteredSearchPaths.Remove(RootDir);

		HandleGameplayTagTreeChanged(true);

		return true;
	}
	return false;
}

void UGameplayTagsManager::GetTagSourceSearchPaths(TArray<FString>& OutPaths)
{
	OutPaths.Reset();
	RegisteredSearchPaths.GenerateKeyArray(OutPaths);
}

int32 UGameplayTagsManager::GetNumTagSourceSearchPaths()
{
	return RegisteredSearchPaths.Num();
}

void UGameplayTagsManager::AddRestrictedGameplayTagSource(const FString& FileName)
{
	FName TagSource = FName(*FPaths::GetCleanFilename(FileName));
	if (TagSource == NAME_None)
	{
		return;
	}

	if (RestrictedGameplayTagSourceNames.Contains(TagSource))
	{
		// Was already added on this pass
		return;
	}

	RestrictedGameplayTagSourceNames.Add(TagSource);
	FGameplayTagSource* FoundSource = FindOrAddTagSource(TagSource, EGameplayTagSourceType::RestrictedTagList);

	// Make sure we have regular tag sources to match the restricted tag sources but don't try to read any tags from them yet.
	FindOrAddTagSource(TagSource, EGameplayTagSourceType::TagList);

	if (FoundSource && FoundSource->SourceRestrictedTagList)
	{
		FoundSource->SourceRestrictedTagList->LoadConfig(URestrictedGameplayTagsList::StaticClass(), *FileName);

#if WITH_EDITOR
		if (GIsEditor || IsRunningCommandlet()) // Sort tags for UI Purposes but don't sort in -game scenario since this would break compat with noneditor cooked builds
		{
			FoundSource->SourceRestrictedTagList->SortTags();
		}
#endif
		for (const FRestrictedGameplayTagTableRow& TableRow : FoundSource->SourceRestrictedTagList->RestrictedGameplayTagList)
		{
			AddTagTableRow(TableRow, TagSource, true);
		}
	}
}

void UGameplayTagsManager::AddTagsFromAdditionalLooseIniFiles(const TArray<FString>& IniFileList)
{
	// Read all tags from the ini
	for (const FString& IniFilePath : IniFileList)
	{
		const FName TagSource = FName(*FPaths::GetCleanFilename(IniFilePath));

		// skip the restricted tag files
		if (RestrictedGameplayTagSourceNames.Contains(TagSource))
		{
			continue;
		}

		UE::TScopeLock Lock(GameplayTagMapCritical);

		FGameplayTagSource* FoundSource = FindOrAddTagSource(TagSource, EGameplayTagSourceType::TagList);

		UE_CLOGF(GAMEPLAYTAGS_VERBOSE, LogGameplayTags, Display, "Loading Tag File: %ls", *IniFilePath);

		if (FoundSource && FoundSource->SourceTagList)
		{
			FoundSource->SourceTagList->ConfigFileName = IniFilePath;

			FoundSource->SourceTagList->LoadConfig(UGameplayTagsList::StaticClass(), *IniFilePath);

			// we don't actually need this in GConfig because they aren't read from again, and they take a lot of memory,
			// and aren't tagged with the plugin name, so can't be unloaded along with the plugin anyway, but
			// since LoadConfig can't take an existing FConfigFile* to load from, we put it into GConfig, then remove it
			GConfig->Remove(IniFilePath);

			FGameplayTagRedirectors::Get().AddRedirectsFromSource(FoundSource);

#if WITH_EDITOR
			if (GIsEditor || IsRunningCommandlet()) // Sort tags for UI Purposes but don't sort in -game scenario since this would break compat with noneditor cooked builds
			{
				FoundSource->SourceTagList->SortTags();
			}
#endif

			for (const FGameplayTagTableRow& TableRow : FoundSource->SourceTagList->GameplayTagList)
			{
				AddTagTableRow(TableRow, TagSource);
			}
		}
	}
}

void UGameplayTagsManager::ConstructGameplayTagTree()
{
	SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree"));
	UE::TScopeLock Lock(GameplayTagMapCritical);
	TGuardValue<bool> GuardRebuilding(bIsConstructingGameplayTagTree, true);
	if (!GameplayRootTag.IsValid())
	{
		GameplayRootTag = MakeShareable(new FGameplayTagNode());

		// Copy invalid characters, then add internal ones
		InvalidTagCharacters = GetDefault<UGameplayTagsSettings>()->InvalidTagCharacters;
		InvalidTagCharacters.Append(TEXT("\r\n\t"));

		// Add prefixes first
		if (ShouldImportTagsFromINI())
		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: ImportINI prefixes"));

			TArray<FString> RestrictedGameplayTagFiles;
			GetRestrictedTagConfigFiles(RestrictedGameplayTagFiles);
			RestrictedGameplayTagFiles.Sort();

			for (const FString& FileName : RestrictedGameplayTagFiles)
			{
				AddRestrictedGameplayTagSource(FileName);
			}
		}

		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: Add native tags"));
			// Add native tags before other tags
			for (FName TagToAdd : LegacyNativeTags)
			{
				AddTagTableRow(FGameplayTagTableRow(TagToAdd), FGameplayTagSource::GetNativeName());
			}

			for (const class FNativeGameplayTag* NativeTag : FNativeGameplayTag::GetRegisteredNativeTags())
			{
				FindOrAddTagSource(NativeTag->GetModuleName(), EGameplayTagSourceType::Native);
				AddTagTableRow(NativeTag->GetGameplayTagTableRow(), NativeTag->GetModuleName());
			}
		}

		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: Construct from data asset"));
			for (UDataTable* DataTable : GameplayTagTables)
			{
				if (DataTable)
				{
					PopulateTreeFromDataTable(DataTable);
				}
			}
		}

		// Create native source
		FindOrAddTagSource(FGameplayTagSource::GetNativeName(), EGameplayTagSourceType::Native);

		if (ShouldImportTagsFromINI())
		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: ImportINI tags"));

#if WITH_EDITOR
			GetMutableDefault<UGameplayTagsSettings>()->SortTags();
#endif

			const UGameplayTagsSettings* Default = GetDefault<UGameplayTagsSettings>();

			FName TagSource = FGameplayTagSource::GetDefaultName();
			FGameplayTagSource* DefaultSource = FindOrAddTagSource(TagSource, EGameplayTagSourceType::DefaultTagList);

			for (const FGameplayTagTableRow& TableRow : Default->GameplayTagList)
			{
				AddTagTableRow(TableRow, TagSource);
			}

			// Make sure default config list is added
			FString DefaultPath = FPaths::ProjectConfigDir() / TEXT("Tags");
			AddTagIniSearchPath(DefaultPath);

			// Refresh any other search paths that need it
			for (TPair<FString, FGameplayTagSearchPathInfo>& Pair : RegisteredSearchPaths)
			{
				if (!Pair.Value.IsValid())
				{
					AddTagIniSearchPath(Pair.Key);
				}
			}
		}

		if (!GIsEditor)
		{
			GConfig->SafeUnloadBranch(*GGameplayTagsIni);
		}

#if WITH_EDITOR
		// Add any transient editor-only tags
		for (FName TransientTag : TransientEditorTags)
		{
			AddTagTableRow(FGameplayTagTableRow(TransientTag), FGameplayTagSource::GetTransientEditorName());
		}
#endif
		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: Request common tags"));

			const UGameplayTagsSettings* Default = GetDefault<UGameplayTagsSettings>();

			// Grab the commonly replicated tags
			CommonlyReplicatedTags.Empty();
			for (FName TagName : Default->CommonlyReplicatedTags)
			{
				if (TagName.IsNone())
				{
					// Still being added to the UI
					continue;
				}

				FGameplayTag Tag = RequestGameplayTag(TagName, false);
				if (Tag.IsValid())
				{
					CommonlyReplicatedTags.Add(Tag);
				}
				else
				{
					UE_LOGF(LogGameplayTags, Warning, "%ls was found in the CommonlyReplicatedTags list but doesn't appear to be a valid tag!", *TagName.ToString());
				}
			}

			bUseFastReplication = Default->FastReplication;
			bUseDynamicReplication = Default->bDynamicReplication;
			bShouldWarnOnInvalidTags = Default->WarnOnInvalidTags;
			NumBitsForContainerSize = Default->NumBitsForContainerSize;
			NetIndexFirstBitSegment = Default->NetIndexFirstBitSegment;

#if WITH_EDITOR
			if (GIsEditor)
			{
				bShouldAllowUnloadingTags = Default->AllowEditorTagUnloading;
			}
			else
#endif
			{
				bShouldAllowUnloadingTags = Default->AllowGameTagUnloading;
			}

		}

		if (ShouldUseFastReplication())
		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: Reconstruct NetIndex"));
			InvalidateNetworkIndex();
		}

		{
			SCOPE_LOG_GAMEPLAYTAGS(TEXT("UGameplayTagsManager::ConstructGameplayTagTree: GameplayTagTreeChangedEvent.Broadcast"));
			BroadcastOnGameplayTagTreeChanged();
		}
	}
}

int32 PrintNetIndiceAssignment = 0;
static FAutoConsoleVariableRef CVarPrintNetIndiceAssignment(TEXT("GameplayTags.PrintNetIndiceAssignment"), PrintNetIndiceAssignment, TEXT("Logs GameplayTag NetIndice assignment"), ECVF_Default );
void UGameplayTagsManager::ConstructNetIndex()
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	bNetworkIndexInvalidated = false;

	NetworkGameplayTagNodeIndex.Empty();

	GameplayTagNodeMap.GenerateValueArray(NetworkGameplayTagNodeIndex);

	NetworkGameplayTagNodeIndex.Sort(FCompareFGameplayTagNodeByTag());

	check(CommonlyReplicatedTags.Num() <= NetworkGameplayTagNodeIndex.Num());

	// Put the common indices up front
	for (int32 CommonIdx=0; CommonIdx < CommonlyReplicatedTags.Num(); ++CommonIdx)
	{
		int32 BaseIdx=0;
		FGameplayTag& Tag = CommonlyReplicatedTags[CommonIdx];

		bool Found = false;
		for (int32 findidx=0; findidx < NetworkGameplayTagNodeIndex.Num(); ++findidx)
		{
			if (NetworkGameplayTagNodeIndex[findidx]->GetCompleteTag() == Tag)
			{
				NetworkGameplayTagNodeIndex.Swap(findidx, CommonIdx);
				Found = true;
				break;
			}
		}

		// A non fatal error should have been thrown when parsing the CommonlyReplicatedTags list. If we make it here, something is seriously wrong.
		checkf( Found, TEXT("Tag %s not found in NetworkGameplayTagNodeIndex"), *Tag.ToString() );
	}

	// This is now sorted and it should be the same on both client and server
	if (NetworkGameplayTagNodeIndex.Num() >= INVALID_TAGNETINDEX)
	{
		ensureMsgf(false, TEXT("Too many tags (%d) in dictionary for networking! Remove tags or increase tag net index size (%d)"), NetworkGameplayTagNodeIndex.Num(), INVALID_TAGNETINDEX);

		NetworkGameplayTagNodeIndex.SetNum(INVALID_TAGNETINDEX - 1);
	}

	InvalidTagNetIndex = IntCastChecked<uint16, int32>(NetworkGameplayTagNodeIndex.Num() + 1);
	NetIndexTrueBitNum = FMath::CeilToInt(FMath::Log2(static_cast<float>(InvalidTagNetIndex)));

	// This should never be smaller than NetIndexTrueBitNum
	NetIndexFirstBitSegment = FMath::Min<int32>(GetDefault<UGameplayTagsSettings>()->NetIndexFirstBitSegment, NetIndexTrueBitNum);

	UE_CLOGF(PrintNetIndiceAssignment, LogGameplayTags, Display, "Assigning NetIndices to %d tags.", NetworkGameplayTagNodeIndex.Num() );

	NetworkGameplayTagNodeIndexHash = 0;

	for (FGameplayTagNetIndex i = 0; i < NetworkGameplayTagNodeIndex.Num(); i++)
	{
		if (NetworkGameplayTagNodeIndex[i].IsValid())
		{
			NetworkGameplayTagNodeIndex[i]->NetIndex = i;

			NetworkGameplayTagNodeIndexHash = FCrc::StrCrc32(*NetworkGameplayTagNodeIndex[i]->GetCompleteTagString().ToLower(), NetworkGameplayTagNodeIndexHash);

			UE_CLOGF(PrintNetIndiceAssignment, LogGameplayTags, Display, "Assigning NetIndex (%d) to Tag (%ls)", i, *NetworkGameplayTagNodeIndex[i]->GetCompleteTag().ToString());
		}
		else
		{
			UE_LOGF(LogGameplayTags, Warning, "TagNode Indice %d is invalid!", i);
		}
	}

	UE_LOGF(LogGameplayTags, Log, "NetworkGameplayTagNodeIndexHash is %x", NetworkGameplayTagNodeIndexHash);
}

FName UGameplayTagsManager::GetTagNameFromNetIndex(FGameplayTagNetIndex Index) const
{
	VerifyNetworkIndex();

	if (Index >= NetworkGameplayTagNodeIndex.Num())
	{
		// Ensure Index is the invalid index. If its higher than that, then something is wrong.
		ensureMsgf(Index == InvalidTagNetIndex, TEXT("Received invalid tag net index %d! Tag index is out of sync on client!"), Index);
		return NAME_None;
	}
	return NetworkGameplayTagNodeIndex[Index]->GetCompleteTagName();
}

FGameplayTagNetIndex UGameplayTagsManager::GetNetIndexFromTag(const FGameplayTag &InTag) const
{
	VerifyNetworkIndex();

	TSharedPtr<FGameplayTagNode> GameplayTagNode = FindTagNode(InTag);

	if (GameplayTagNode.IsValid())
	{
		return GameplayTagNode->GetNetIndex();
	}

	return InvalidTagNetIndex;
}

void UGameplayTagsManager::PushDeferOnGameplayTagTreeChangedBroadcast()
{
	++DeferBroadcastOnGameplayTagTreeChanged;
}

void UGameplayTagsManager::PopDeferOnGameplayTagTreeChangedBroadcast()
{
	--DeferBroadcastOnGameplayTagTreeChanged;
	ensure(DeferBroadcastOnGameplayTagTreeChanged >= 0);
	if (DeferBroadcastOnGameplayTagTreeChanged == 0 && bBroadcastOnGameplayTagTreeChangedWasDeferred)
	{
		bBroadcastOnGameplayTagTreeChangedWasDeferred = false;
		BroadcastOnGameplayTagTreeChanged();
	}
}

void UGameplayTagsManager::PushDeferGameplayTagTreeRebuilds()
{
	++DeferGameplayTagTreeRebuilds;
}

void UGameplayTagsManager::PopDeferGameplayTagTreeRebuilds()
{
	--DeferGameplayTagTreeRebuilds;
	ensure(DeferGameplayTagTreeRebuilds >= 0);
	if (DeferGameplayTagTreeRebuilds == 0 && bGameplayTagTreeRebuildWasDeferred)
	{
		bGameplayTagTreeRebuildWasDeferred = false;
		HandleGameplayTagTreeChanged(true);
	}
}

bool UGameplayTagsManager::ShouldImportTagsFromINI() const
{
	return GetDefault<UGameplayTagsSettings>()->ImportTagsFromConfig;
}

bool UGameplayTagsManager::ShouldUnloadTags() const
{
#if WITH_EDITOR
	if (bShouldAllowUnloadingTags && GIsEditor && GEngine)
	{
		// Check if we have an active PIE index without linking GEditor, and compare to game setting
		FWorldContext* PIEWorldContext = GEngine->GetWorldContextFromPIEInstance(0);
		UGameplayTagsSettings* MutableDefault = GetMutableDefault<UGameplayTagsSettings>();

		if (PIEWorldContext && !MutableDefault->AllowGameTagUnloading)
		{
			UE_LOGF(LogGameplayTags, Warning, "Ignoring request to unload tags during Play In Editor because AllowGameTagUnloading=false");
			return false;
		}
	}
#endif

	if (ShouldAllowUnloadingTagsOverride.IsSet())
	{
		return ShouldAllowUnloadingTagsOverride.GetValue();
	}

	return bShouldAllowUnloadingTags;
}

void UGameplayTagsManager::SetShouldUnloadTagsOverride(bool bShouldUnloadTags)
{
	ShouldAllowUnloadingTagsOverride = bShouldUnloadTags;
}

void UGameplayTagsManager::ClearShouldUnloadTagsOverride()
{
	ShouldAllowUnloadingTagsOverride.Reset();
}

void UGameplayTagsManager::SetShouldDeferGameplayTagTreeRebuilds(bool bShouldDeferRebuilds)
{
	bool bWasDeferred = ShouldDeferGameplayTagTreeRebuilds.IsSet() && ShouldDeferGameplayTagTreeRebuilds.GetValue();

	ShouldDeferGameplayTagTreeRebuilds = bShouldDeferRebuilds;

	if (!bWasDeferred && bShouldDeferRebuilds)
	{
		PushDeferGameplayTagTreeRebuilds();
	}
	else if (bWasDeferred && !bShouldDeferRebuilds)
	{
		PopDeferGameplayTagTreeRebuilds();
	}
}

void UGameplayTagsManager::ClearShouldDeferGameplayTagTreeRebuilds(bool bRebuildTree)
{
	bool bWasDeferred = ShouldDeferGameplayTagTreeRebuilds.IsSet() && ShouldDeferGameplayTagTreeRebuilds.GetValue();

	ShouldDeferGameplayTagTreeRebuilds.Reset();

	if (bWasDeferred)
	{
		// This will do a rebuild if one was queued and there were no other deferrals
		PopDeferGameplayTagTreeRebuilds();
	} 
}

void UGameplayTagsManager::GetRestrictedTagConfigFiles(TArray<FString>& RestrictedConfigFiles) const
{
	const UGameplayTagsSettings* Default = GetDefault<UGameplayTagsSettings>();

	if (Default)
	{
		for (const FRestrictedConfigInfo& Config : Default->RestrictedConfigFiles)
		{
			RestrictedConfigFiles.Add(FString::Printf(TEXT("%sTags/%s"), *FPaths::SourceConfigDir(), *Config.RestrictedConfigName));
		}
	}

	for (const TPair<FString, FGameplayTagSearchPathInfo>& Pair : RegisteredSearchPaths)
	{
		for (const FString& IniFilePath : Pair.Value.TagIniList)
		{
			TArray<FRestrictedConfigInfo> IniRestrictedConfigs;
			GameplayTagUtil::GetRestrictedConfigsFromIni(IniFilePath, IniRestrictedConfigs);
			for (const FRestrictedConfigInfo& Config : IniRestrictedConfigs)
			{
				RestrictedConfigFiles.Add(FString::Printf(TEXT("%s/%s"), *FPaths::GetPath(IniFilePath), *Config.RestrictedConfigName));
			}
		}
	}
}

void UGameplayTagsManager::GetRestrictedTagSources(TArray<const FGameplayTagSource*>& Sources) const
{
	for (const TPair<FName, FGameplayTagSource>& Pair : TagSources)
	{
		if (Pair.Value.SourceType == EGameplayTagSourceType::RestrictedTagList)
		{
			Sources.Add(&Pair.Value);
		}
	}
}

void UGameplayTagsManager::GetOwnersForTagSource(const FString& SourceName, TArray<FString>& OutOwners) const
{
	const UGameplayTagsSettings* Default = GetDefault<UGameplayTagsSettings>();

	if (Default)
	{
		for (const FRestrictedConfigInfo& Config : Default->RestrictedConfigFiles)
		{
			if (Config.RestrictedConfigName.Equals(SourceName))
			{
				OutOwners = Config.Owners;
				return;
			}
		}
	}
}

void UGameplayTagsManager::GameplayTagContainerLoaded(FGameplayTagContainer& Container, FProperty* SerializingProperty) const
{
	RedirectTagsForContainer(Container, SerializingProperty);

	if (OnGameplayTagLoadedDelegate.IsBound())
	{
		for (const FGameplayTag& Tag : Container)
		{
			OnGameplayTagLoadedDelegate.Broadcast(Tag);
		}
	}
}

void UGameplayTagsManager::SingleGameplayTagLoaded(FGameplayTag& Tag, FProperty* SerializingProperty) const
{
	RedirectSingleGameplayTag(Tag, SerializingProperty);

	OnGameplayTagLoadedDelegate.Broadcast(Tag);
}

void UGameplayTagsManager::RedirectTagsForContainer(FGameplayTagContainer& Container, FProperty* SerializingProperty) const
{
	TArray<FName> NamesToRemove;
	TArray<FGameplayTag> TagsToAdd;

	// First populate the NamesToRemove and TagsToAdd sets by finding tags in the container that have redirects
	{
		// Lock the tag map to safely iterate over gameplay tags using non-thread-safe functions
		UE::TScopeLock Lock(GameplayTagMapCritical);

		for (auto TagIt = Container.CreateConstIterator(); TagIt; ++TagIt)
		{
			const FName TagName = TagIt->GetTagName();

			FGameplayTag NewTag;
			if (FGameplayTagRedirectors::Get().RedirectTag(TagName, NewTag))
			{
				NamesToRemove.Add(TagName);
				if (NewTag.IsValid())
				{
					TagsToAdd.Add(MoveTemp(NewTag));
				}
			}
		#if WITH_EDITOR
			else if (SerializingProperty)
			{
				// Warn about invalid tags at load time in editor builds, too late to fix it in cooked builds
				FGameplayTag OldTag = RequestGameplayTag(TagName, false);
				if (!OldTag.IsValid())
				{
					if (ShouldWarnOnInvalidTags())
					{
						FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
						UObject* LoadingObject = LoadContext ? LoadContext->SerializedObject : nullptr;
						UPackage* LoadingPackage = LoadingObject ? LoadingObject->GetPackage() : nullptr;
						UE_ASSET_LOG(LogGameplayTags, Warning, *GetPathNameSafe(LoadingObject), TEXT("Invalid GameplayTag %s found in property %s. Package: %s"), *TagName.ToString(), *GetPathNameSafe(SerializingProperty), *GetNameSafe(LoadingPackage));
					}
				}
			}
		#endif
		}
		
	}

	// Remove all tags from the NamesToRemove set
	for (FName RemoveName : NamesToRemove)
	{
		Container.RemoveTag(FGameplayTag(RemoveName));
	}

	// Add all tags from the TagsToAdd set
	for (FGameplayTag& AddTag : TagsToAdd)
	{
		Container.AddTag(MoveTemp(AddTag));
	}
}

void UGameplayTagsManager::RedirectSingleGameplayTag(FGameplayTag& Tag, FProperty* SerializingProperty) const
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	const FName TagName = Tag.GetTagName();
	FGameplayTag NewTag;
	if (FGameplayTagRedirectors::Get().RedirectTag(TagName, NewTag))
	{
		if (NewTag.IsValid())
		{
			Tag = NewTag;
		}
	}
#if WITH_EDITOR
	else if (!TagName.IsNone() && SerializingProperty)
	{
		// Warn about invalid tags at load time in editor builds, too late to fix it in cooked builds
		FGameplayTag OldTag = RequestGameplayTag(TagName, false);
		if (!OldTag.IsValid())
		{
			if (ShouldWarnOnInvalidTags())
			{
				FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
				UObject* LoadingObject = LoadContext ? LoadContext->SerializedObject : nullptr;
				UPackage* LoadingPackage = LoadingObject ? LoadingObject->GetPackage() : nullptr;
				UE_ASSET_LOG(LogGameplayTags, Warning, *GetPathNameSafe(LoadingObject), TEXT("Invalid GameplayTag %s found in property %s. Package:%s"), *TagName.ToString(), *GetPathNameSafe(SerializingProperty), *GetNameSafe(LoadingPackage));
			}			
		}
	}
#endif
}

bool UGameplayTagsManager::ImportSingleGameplayTag(FGameplayTag& Tag, FName ImportedTagName, bool bImportFromSerialize) const
{
	// None is always valid, no need to do any real work.
	if (ImportedTagName == NAME_None)
	{
		return true;
	}

	UE::TScopeLock Lock(GameplayTagMapCritical);
	bool bRetVal = false;
	FGameplayTag RedirectedTag;
	if (FGameplayTagRedirectors::Get().RedirectTag(ImportedTagName, RedirectedTag))
	{
		Tag = RedirectedTag;
		bRetVal = true;
	}
	else if (ValidateTagCreation(ImportedTagName))
	{
		// The tag name is valid
		Tag.TagName = ImportedTagName;
		bRetVal = true;
	}

	if (!bRetVal && bImportFromSerialize && !ImportedTagName.IsNone())
	{
#if WITH_EDITOR
		if (ShouldWarnOnInvalidTags())
		{
			// These are more elaborate checks to ensure we're actually loading a UObject, and not pasting it, compiling it, or other possible paths into this function.
			const FUObjectSerializeContext* LoadContext = FUObjectThreadContext::Get().GetSerializeContext();
			const UObject* LoadingObject = LoadContext ? LoadContext->SerializedObject : nullptr;
			UPackage* LoadingPackage = LoadingObject ? LoadingObject->GetPackage() : nullptr;
			if (LoadingObject)
			{
				// We need to defer the check until after native gameplay tags are done loading (in case the tag has not yet been defined)
				CallOrRegister_OnDoneAddingNativeTagsDelegate(FSimpleMulticastDelegate::FDelegate::CreateWeakLambda(this,
					[this, ImportedTagName, AssetName = GetPathNameSafe(LoadingObject), FullObjectPath = LoadingObject->GetFullName(), PackageNameString = GetNameSafe(LoadingPackage)]()
					{
						// Verify it again -- it could have been a late-loading native tag
						if (!ValidateTagCreation(ImportedTagName))
						{
							UE_ASSET_LOG(LogGameplayTags, Warning, *AssetName, TEXT("Invalid GameplayTag %s found in object %s. Package: %s"), *ImportedTagName.ToString(), *FullObjectPath, *PackageNameString);
						}
					}));
			}
		}
#endif
		// For imported tags that are part of a serialize, leave invalid ones the same way normal serialization does to avoid data loss
		Tag.TagName = ImportedTagName;
		bRetVal = true;
	}

	if (bRetVal)
	{
		OnGameplayTagLoadedDelegate.Broadcast(Tag);
	}
	else
	{
		// No valid tag established in this attempt
		Tag.TagName = NAME_None;
	}

	return bRetVal;
}

UE_AUTORTFM_ALWAYS_OPEN void UGameplayTagsManager::InitializeManager()
{
	// This should never be called during shutdown, use GetIfAllocated
	check(!IsEngineExitRequested());
	check(!SingletonManager);

	SCOPED_BOOT_TIMING("UGameplayTagsManager::InitializeManager");
	SCOPE_LOG_TIME_IN_SECONDS(TEXT("UGameplayTagsManager::InitializeManager"), nullptr);

	SingletonManager = NewObject<UGameplayTagsManager>(GetTransientPackage(), NAME_None);
	SingletonManager->AddToRoot();

	SingletonManager->ChildNodesCache.Empty(CVarGameplayTagsLruCacheSize.GetValueOnAnyThread());

	//This is always going to be a synchronous load this early in init, so save some time by not attempting anything async
	SingletonManager->LoadGameplayTagTables(false);
	SingletonManager->ConstructGameplayTagTree();

	// Bind to end of engine init to be done adding native tags
	FCoreDelegates::GetOnPostEngineInit().AddUObject(SingletonManager, &UGameplayTagsManager::DoneAddingNativeTags);
}

void UGameplayTagsManager::PopulateTreeFromDataTable(class UDataTable* InTable)
{
	checkf(GameplayRootTag.IsValid(), TEXT("ConstructGameplayTagTree() must be called before PopulateTreeFromDataTable()"));
	static const FString ContextString("UGameplayTagsManager::PopulateTreeFromDataTable");
	
	TArray<FGameplayTagTableRow*> TagTableRows;
	InTable->GetAllRows<FGameplayTagTableRow>(ContextString, TagTableRows);

	FName SourceName = InTable->GetOutermost()->GetFName();

	FGameplayTagSource* FoundSource = FindOrAddTagSource(SourceName, EGameplayTagSourceType::DataTable);

	for (const FGameplayTagTableRow* TagRow : TagTableRows)
	{
		if (TagRow)
		{
			AddTagTableRow(*TagRow, SourceName);
		}
	}
}

void UGameplayTagsManager::AddTagTableRow(const FGameplayTagTableRow& TagRow, FName SourceName, bool bIsRestrictedTag)
{
	TSharedPtr<FGameplayTagNode> CurNode = GameplayRootTag;
	TArray<TSharedPtr<FGameplayTagNode>, TInlineAllocator<6>> AncestorNodes;
	bool bAllowNonRestrictedChildren = true;

	const FRestrictedGameplayTagTableRow* RestrictedTagRow = static_cast<const FRestrictedGameplayTagTableRow*>(&TagRow);
	if (bIsRestrictedTag && RestrictedTagRow)
	{
		bAllowNonRestrictedChildren = RestrictedTagRow->bAllowNonRestrictedChildren;
	}

	// Split the tag text on the "." delimiter to establish tag depth and then insert each tag into the gameplay tag tree
	// We try to avoid as many string->FName conversions as possible as they are slow
	FName OriginalTagName = TagRow.Tag;
	FNameBuilder FullTagString(OriginalTagName);

#if WITH_EDITOR
	{
		// In editor builds, validate string
		// These must get fixed up cooking to work properly
		FText ErrorText;
		FNameBuilder FixedString;

		if (!IsValidGameplayTagString(FullTagString, &ErrorText, &FixedString))
		{
			if (FixedString.Len() == 0)
			{
				// No way to fix it
				UE_LOGF(LogGameplayTags, Error, "Invalid tag %ls from source %ls: %ls!", *FullTagString, *SourceName.ToString(), *ErrorText.ToString());
				return;
			}
			else
			{
				UE_LOGF(LogGameplayTags, Error, "Invalid tag %ls from source %ls: %ls! Replacing with %ls, you may need to modify InvalidTagCharacters", *FullTagString, *SourceName.ToString(), *ErrorText.ToString(), *FixedString);
				FullTagString.Reset();
				FullTagString << FixedString;
				OriginalTagName = FName(*FixedString);
			}
		}
	}
#endif

	struct FRequiredTag
	{
		FName ShortTagName;
		FName FullTagName;
		bool bIsExplicitTag;
	};

	TArray<FRequiredTag, TInlineAllocator<6>> RequiredTags;

	{
		// don't need to lock if we are constructing the gameplay tag tree as it's already handled in an outer scope
		UE::TConditionalScopeLock Lock(GameplayTagMapCritical, !bIsConstructingGameplayTagTree);

		constexpr FAsciiSet Period(".");

		// first try and push parents to see how far back we need to go, to be able to early out and not over-process the parent nodes
		// and avoids unnecessary String -> FName lookup
		const FStringView FullTagView(FullTagString);
		FStringView Remainder(FullTagView);
		while (!Remainder.IsEmpty())
		{
			const FStringView CurrentFullTag = Remainder;

			const FStringView SubTag = FAsciiSet::FindSuffixWithout(Remainder, Period);
			Remainder.LeftChopInline(SubTag.Len());

			// Skip the delimiter, if present.
			const FStringView Skip = FAsciiSet::FindSuffixWith(Remainder, Period);
			Remainder.LeftChopInline(Skip.Len());

			// Skip any empty sub tags in the hierarchy.
			if (SubTag.IsEmpty())
			{
				continue;
			}

			const bool bIsExplicitTag = CurrentFullTag.Len() == FullTagView.Len();
			const FName FullTagName = bIsExplicitTag ? OriginalTagName : FName(CurrentFullTag);
			
			// editor builds need everything to be pushed in order to track additional data (SourceName associated with every node)
#if !WITH_EDITOR
			if (!bIsExplicitTag) // we assume the most explicit tag isn't in the tree already
			{
				TSharedPtr<FGameplayTagNode> *FoundNode = GameplayTagNodeMap.Find(FGameplayTag(FullTagName));
				if (FoundNode && FoundNode->IsValid())
				{
					// this early out leaves us with only missing tags from the tree in RequiredTags and CurNode set to the parent to start pushing into
					CurNode = *FoundNode;
					break;
				}
			}
#endif

			RequiredTags.Emplace(
				FRequiredTag
				{
					.ShortTagName = FName(SubTag),
					.FullTagName = FullTagName,
					.bIsExplicitTag = bIsExplicitTag
				}
			);
		}
	}

	bool bHasSeenConflict = false;

	// process backwards as required tags is a LIFO stack
	while (!RequiredTags.IsEmpty())
	{
		const FRequiredTag CurrentTag = RequiredTags.Pop();

		TArray< TSharedPtr<FGameplayTagNode> >& ChildTags = CurNode.Get()->GetChildTagNodes();		
		int32 InsertionIdx = InsertTagIntoNodeArray(CurrentTag.ShortTagName, CurrentTag.FullTagName, CurNode, ChildTags, SourceName, TagRow.DevComment, CurrentTag.bIsExplicitTag, bIsRestrictedTag, bAllowNonRestrictedChildren);
		CurNode = ChildTags[InsertionIdx];

		// Tag conflicts only affect the editor so we don't look for them in the game
#if WITH_EDITORONLY_DATA
		if (bIsRestrictedTag)
		{
			CurNode->bAncestorHasConflict = bHasSeenConflict;

			// If the sources don't match and the tag is explicit and we should've added the tag explicitly here, we have a conflict
			if (CurNode->GetFirstSourceName() != SourceName && (CurNode->bIsExplicitTag && CurrentTag.bIsExplicitTag))
			{
				// mark all ancestors as having a bad descendant
				for (TSharedPtr<FGameplayTagNode> CurAncestorNode : AncestorNodes)
				{
					CurAncestorNode->bDescendantHasConflict = true;
				}

				// mark the current tag as having a conflict
				CurNode->bNodeHasConflict = true;

				// append source names
				CurNode->SourceNames.Add(SourceName);

				// mark all current descendants as having a bad ancestor
				MarkChildrenOfNodeConflict(CurNode);
			}

			// mark any children we add later in this function as having a bad ancestor
			if (CurNode->bNodeHasConflict)
			{
				bHasSeenConflict = true;
			}

			AncestorNodes.Add(CurNode);
		}
#endif
	}
}

void UGameplayTagsManager::MarkChildrenOfNodeConflict(TSharedPtr<FGameplayTagNode> CurNode)
{
#if WITH_EDITORONLY_DATA
	TArray< TSharedPtr<FGameplayTagNode> >& ChildTags = CurNode.Get()->GetChildTagNodes();
	for (TSharedPtr<FGameplayTagNode> ChildNode : ChildTags)
	{
		ChildNode->bAncestorHasConflict = true;
		MarkChildrenOfNodeConflict(ChildNode);
	}
#endif
}

void UGameplayTagsManager::BroadcastOnGameplayTagTreeChanged()
{
	if (DeferBroadcastOnGameplayTagTreeChanged > 0)
	{
		bBroadcastOnGameplayTagTreeChangedWasDeferred = true;
	}
	else
	{
		IGameplayTagsModule::OnGameplayTagTreeChanged.Broadcast();
	}
}

void UGameplayTagsManager::HandleGameplayTagTreeChanged(bool bRecreateTree)
{
	// Don't do anything during a reconstruct or before initial native tags are done loading
	if (!bIsConstructingGameplayTagTree && bDoneAddingNativeTags)
	{
		if (bRecreateTree && DeferGameplayTagTreeRebuilds > 0)
		{
			// Defer the full rebuild but do the simple one
			bGameplayTagTreeRebuildWasDeferred = true;
			bRecreateTree = false;
		}

		if (bRecreateTree)
		{
#if WITH_EDITOR
			if (GIsEditor)
			{
				// In the editor refresh everything
				EditorRefreshGameplayTagTree();
				return;
			}
#endif
			DestroyGameplayTagTree();
			ConstructGameplayTagTree();
		}
		else
		{
			// Refresh if we're done adding tags
			if (ShouldUseFastReplication())
			{
				InvalidateNetworkIndex();
			}

			BroadcastOnGameplayTagTreeChanged();
		}
	}
	else if (bRecreateTree)
	{
		bNeedsTreeRebuildOnDoneAddingGameplayTags = true;
	}
}

UGameplayTagsManager::~UGameplayTagsManager()
{
	DestroyGameplayTagTree();
	SingletonManager = nullptr;
}

void UGameplayTagsManager::DestroyGameplayTagTree()
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	if (GameplayRootTag.IsValid())
	{
		GameplayRootTag->ResetNode();
		GameplayRootTag.Reset();
		GameplayTagNodeMap.Reset();
	}
	RestrictedGameplayTagSourceNames.Reset();

	for (TPair<FString, FGameplayTagSearchPathInfo>& Pair : RegisteredSearchPaths)
	{
		Pair.Value.bWasAddedToTree = false;
	}

	ChildNodesCache.Empty(ChildNodesCache.Max());
}

int32 UGameplayTagsManager::InsertTagIntoNodeArray(FName Tag, FName FullTag, TSharedPtr<FGameplayTagNode> ParentNode, TArray< TSharedPtr<FGameplayTagNode> >& NodeArray, FName SourceName, const FString& DevComment, bool bIsExplicitTag, bool bIsRestrictedTag, bool bAllowNonRestrictedChildren)
{
	int32 FoundNodeIdx = INDEX_NONE;
	int32 WhereToInsert = INDEX_NONE;

	// See if the tag is already in the array

	// LowerBoundBy returns Position of the first element >= Value, may be position after last element in range
	int32 LowerBoundIndex = Algo::LowerBoundBy(NodeArray, Tag,
		[](const TSharedPtr<FGameplayTagNode>& N) -> FName { return N->GetSimpleTagName(); },
		[](const FName& A, const FName& B) { return A != B && UE::ComparisonUtility::CompareWithNumericSuffix(A, B) < 0; });

	if (LowerBoundIndex < NodeArray.Num())
	{
		FGameplayTagNode* CurrNode = NodeArray[LowerBoundIndex].Get();
		if (CurrNode->GetSimpleTagName() == Tag)
		{
			FoundNodeIdx = LowerBoundIndex;
#if WITH_EDITORONLY_DATA
			// If we are explicitly adding this tag then overwrite the existing children restrictions with whatever is in the ini
			// If we restrict children in the input data, make sure we restrict them in the existing node. This applies to explicit and implicitly defined nodes
			if (bAllowNonRestrictedChildren == false || bIsExplicitTag)
			{
				// check if the tag is explicitly being created in more than one place.
				if (CurrNode->bIsExplicitTag && bIsExplicitTag)
				{
					// restricted tags always get added first
					// 
					// There are two possibilities if we're adding a restricted tag. 
					// If the existing tag is non-restricted the restricted tag should take precedence. This may invalidate some child tags of the existing tag.
					// If the existing tag is restricted we have a conflict. This is explicitly not allowed.
					if (bIsRestrictedTag)
					{

					}
				}
				CurrNode->bAllowNonRestrictedChildren = bAllowNonRestrictedChildren;
				CurrNode->bIsExplicitTag = CurrNode->bIsExplicitTag || bIsExplicitTag;
			}
#endif				
		}
		else
		{
			// Insert new node before this
			WhereToInsert = LowerBoundIndex;
		}
	}

	if (FoundNodeIdx == INDEX_NONE)
	{
		if (WhereToInsert == INDEX_NONE)
		{
			// Insert at end
			WhereToInsert = NodeArray.Num();
		}

		// Don't add the root node as parent
		TSharedPtr<FGameplayTagNode> TagNode = MakeShareable(new FGameplayTagNode(Tag, FullTag, ParentNode != GameplayRootTag ? ParentNode : nullptr, bIsExplicitTag, bIsRestrictedTag, bAllowNonRestrictedChildren));

		// Add at the sorted location
		FoundNodeIdx = NodeArray.Insert(TagNode, WhereToInsert);

		FGameplayTag GameplayTag = TagNode->GetCompleteTag();

		// These should always match
		ensure(GameplayTag.GetTagName() == FullTag);

		{
			// This critical section is to handle an issue where tag requests come from another thread when async loading from a background thread in FGameplayTagContainer::Serialize.
			// This function is not generically threadsafe.
			UE::TConditionalScopeLock Lock(GameplayTagMapCritical, !bIsConstructingGameplayTagTree);
			GameplayTagNodeMap.Add(GameplayTag, TagNode);

			// Invalidate the cache behind the lock
			ChildNodesCache.Empty(ChildNodesCache.Max());
		}
	}

#if WITH_EDITOR
	// Set/update editor only data
	NodeArray[FoundNodeIdx]->SourceNames.AddUnique(SourceName);

	if (NodeArray[FoundNodeIdx]->DevComment.IsEmpty() && !DevComment.IsEmpty())
	{
		NodeArray[FoundNodeIdx]->DevComment = DevComment;
	}
#endif

	return FoundNodeIdx;
}

void UGameplayTagsManager::PrintReplicationIndices()
{
	VerifyNetworkIndex();

	UE_LOGF(LogGameplayTags, Display, "::PrintReplicationIndices (TOTAL %d)", GameplayTagNodeMap.Num());

	UE::TScopeLock Lock(GameplayTagMapCritical);

	for (auto It : GameplayTagNodeMap)
	{
		FGameplayTag Tag = It.Key;
		TSharedPtr<FGameplayTagNode> Node = It.Value;

		UE_LOGF(LogGameplayTags, Display, "Tag %ls NetIndex: %d", *Tag.ToString(), Node->GetNetIndex());
	}

}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
void UGameplayTagsManager::PrintReplicationFrequencyReport()
{
	VerifyNetworkIndex();

	UE_LOGF(LogGameplayTags, Warning, "=================================");
	UE_LOGF(LogGameplayTags, Warning, "Gameplay Tags Replication Report");

	UE_LOGF(LogGameplayTags, Warning, "\nTags replicated solo:");
	ReplicationCountMap_SingleTags.ValueSort(TGreater<int32>());
	for (auto& It : ReplicationCountMap_SingleTags)
	{
		UE_LOGF(LogGameplayTags, Warning, "%ls - %d", *It.Key.ToString(), It.Value);
	}
	
	// ---------------------------------------

	UE_LOGF(LogGameplayTags, Warning, "\nTags replicated in containers:");
	ReplicationCountMap_Containers.ValueSort(TGreater<int32>());
	for (auto& It : ReplicationCountMap_Containers)
	{
		UE_LOGF(LogGameplayTags, Warning, "%ls - %d", *It.Key.ToString(), It.Value);
	}

	// ---------------------------------------

	UE_LOGF(LogGameplayTags, Warning, "\nAll Tags replicated:");
	ReplicationCountMap.ValueSort(TGreater<int32>());
	for (auto& It : ReplicationCountMap)
	{
		UE_LOGF(LogGameplayTags, Warning, "%ls - %d", *It.Key.ToString(), It.Value);
	}

	TMap<int32, int32> SavingsMap;
	int32 BaselineCost = 0;
	for (int32 Bits=1; Bits < NetIndexTrueBitNum; ++Bits)
	{
		int32 TotalSavings = 0;
		BaselineCost = 0;

		FGameplayTagNetIndex ExpectedNetIndex=0;
		for (auto& It : ReplicationCountMap)
		{
			int32 ExpectedCostBits = 0;
			bool FirstSeg = ExpectedNetIndex < FMath::Pow(2.f, Bits);
			if (FirstSeg)
			{
				// This would fit in the first Bits segment
				ExpectedCostBits = Bits+1;
			}
			else
			{
				// Would go in the second segment, so we pay the +1 cost
				ExpectedCostBits = NetIndexTrueBitNum+1;
			}

			int32 Savings = (NetIndexTrueBitNum - ExpectedCostBits) * It.Value;
			BaselineCost += NetIndexTrueBitNum * It.Value;

			//UE_LOGF(LogGameplayTags, Warning, "[Bits: %d] Tag %ls would save %d bits", Bits, *It.Key.ToString(), Savings);
			ExpectedNetIndex++;
			TotalSavings += Savings;
		}

		SavingsMap.FindOrAdd(Bits) = TotalSavings;
	}

	SavingsMap.ValueSort(TGreater<int32>());
	int32 BestBits = 0;
	for (auto& It : SavingsMap)
	{
		if (BestBits == 0)
		{
			BestBits = It.Key;
		}

		UE_LOGF(LogGameplayTags, Warning, "%d bits would save %d (%.2f)", It.Key, It.Value, (float)It.Value / (float)BaselineCost);
	}

	UE_LOGF(LogGameplayTags, Warning, "\nSuggested config:");

	// Write out a nice copy pastable config
	int32 Count=0;
	for (auto& It : ReplicationCountMap)
	{
		UE_LOGF(LogGameplayTags, Warning, "+CommonlyReplicatedTags=%ls", *It.Key.ToString());

		if (Count == FMath::Pow(2.f, BestBits))
		{
			// Print a blank line out, indicating tags after this are not necessary but still may be useful if the user wants to manually edit the list.
			UE_LOGF(LogGameplayTags, Warning, "");
		}

		if (Count++ >= FMath::Pow(2.f, BestBits+1))
		{
			break;
		}
	}

	UE_LOGF(LogGameplayTags, Warning, "NetIndexFirstBitSegment=%d", BestBits);

	UE_LOGF(LogGameplayTags, Warning, "=================================");
}

void UGameplayTagsManager::NotifyTagReplicated(FGameplayTag Tag, bool WasInContainer)
{
	ReplicationCountMap.FindOrAdd(Tag)++;

	if (WasInContainer)
	{
		ReplicationCountMap_Containers.FindOrAdd(Tag)++;
	}
	else
	{
		ReplicationCountMap_SingleTags.FindOrAdd(Tag)++;
	}
	
}
#endif

#if WITH_EDITOR

static void RecursiveRootTagSearch(const FString& InFilterString, const TArray<TSharedPtr<FGameplayTagNode> >& GameplayRootTags, TArray< TSharedPtr<FGameplayTagNode> >& OutTagArray)
{
	FString CurrentFilter, RestOfFilter;
	if (!InFilterString.Split(TEXT("."), &CurrentFilter, &RestOfFilter))
	{
		CurrentFilter = InFilterString;
	}

	for (int32 iTag = 0; iTag < GameplayRootTags.Num(); ++iTag)
	{
		FString RootTagName = GameplayRootTags[iTag].Get()->GetSimpleTagName().ToString();

		if (RootTagName == CurrentFilter)
		{
			if (RestOfFilter.IsEmpty())
			{
				// We've reached the end of the filter, add tags
				OutTagArray.Add(GameplayRootTags[iTag]);
			}
			else
			{
				// Recurse into our children
				RecursiveRootTagSearch(RestOfFilter, GameplayRootTags[iTag]->GetChildTagNodes(), OutTagArray);
			}
		}		
	}
}

void UGameplayTagsManager::GetFilteredGameplayRootTags(const FString& InFilterString, TArray< TSharedPtr<FGameplayTagNode> >& OutTagArray) const
{
	TArray<FString> PreRemappedFilters;
	TArray<FString> Filters;
	TArray<TSharedPtr<FGameplayTagNode>>& GameplayRootTags = GameplayRootTag->GetChildTagNodes();

	OutTagArray.Empty();
	if( InFilterString.ParseIntoArray( PreRemappedFilters, TEXT( "," ), true ) > 0 )
	{
		const UGameplayTagsSettings* CDO = GetDefault<UGameplayTagsSettings>();
		for (FString& Str : PreRemappedFilters)
		{
			bool Remapped = false;
			for (const FGameplayTagCategoryRemap& RemapInfo : CDO->CategoryRemapping)
			{
				if (RemapInfo.BaseCategory == Str)
				{
					Remapped = true;
					Filters.Append(RemapInfo.RemapCategories);
				}
			}
			if (Remapped == false)
			{
				Filters.Add(Str);
			}
		}		

		// Check all filters in the list
		for (int32 iFilter = 0; iFilter < Filters.Num(); ++iFilter)
		{
			RecursiveRootTagSearch(Filters[iFilter], GameplayRootTags, OutTagArray);
		}

		if (OutTagArray.Num() == 0)
		{
			// We had filters but nothing matched. Ignore the filters.
			// This makes sense to do with engine level filters that games can optionally specify/override.
			// We never want to impose tag structure on projects, but still give them the ability to do so for their project.
			OutTagArray = GameplayRootTags;
		}

	}
	else
	{
		// No Filters just return them all
		OutTagArray = GameplayRootTags;
	}
}

FString UGameplayTagsManager::GetCategoriesMetaFromPropertyHandle(TSharedPtr<IPropertyHandle> PropertyHandle) const
{
	// Global delegate override. Useful for parent structs that want to override tag categories based on their data (e.g. not static property meta data)
	FString DelegateOverrideString;
	OnGetCategoriesMetaFromPropertyHandle.Broadcast(PropertyHandle, DelegateOverrideString);
	if (DelegateOverrideString.IsEmpty() == false)
	{
		return DelegateOverrideString;
	}

	return StaticGetCategoriesMetaFromPropertyHandle(PropertyHandle);
}

FString UGameplayTagsManager::StaticGetCategoriesMetaFromPropertyHandle(TSharedPtr<class IPropertyHandle> PropertyHandle)
{
	FString Categories;

	while(PropertyHandle.IsValid())
	{
		if (FProperty* Property = PropertyHandle->GetProperty())
		{
			/**
			 *	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta = (Categories="GameplayCue"))
			 *	FGameplayTag GameplayCueTag;
			 */
			Categories = GetCategoriesMetaFromField(Property);
			if (!Categories.IsEmpty())
			{
				break;
			}

			/**
			 *	USTRUCT(meta=(Categories="EventKeyword"))
			 *	struct FGameplayEventKeywordTag : public FGameplayTag
			 */
			if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				Categories = GetCategoriesMetaFromField<UScriptStruct>(StructProperty->Struct);
				if (!Categories.IsEmpty())
				{
					break;
				}
			}

			/**	TArray<FGameplayEventKeywordTag> QualifierTagTestList; */
			if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				Categories = GetCategoriesMetaFromField(ArrayProperty->Inner);
				if (!Categories.IsEmpty())
				{
					break;
				}
			}

			/**	TMap<FGameplayTag, ValueType> GameplayTagMap; */
			if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
			{
				Categories = GetCategoriesMetaFromField(MapProperty->KeyProp);
				if (!Categories.IsEmpty())
				{
					break;
				}
			}
		}

		TSharedPtr<IPropertyHandle> ParentHandle = PropertyHandle->GetParentHandle();
		
		if (ParentHandle.IsValid())
		{
			/** Check if the parent handle's base class is of the same class. It's possible the current child property is from a subobject which in that case we probably want to ignore
			  * any meta category restrictions coming from any parent properties. A subobject's gameplay tag property without any declared meta categories should stay that way. */
			if (PropertyHandle->GetOuterBaseClass() != ParentHandle->GetOuterBaseClass())
			{
				break;
			}
		}

		PropertyHandle = ParentHandle;
	}
	
	return Categories;
}

FString UGameplayTagsManager::GetCategoriesMetaFromFunction(const UFunction* ThisFunction, FName ParamName /** = NAME_None */)
{
	FString FilterString;
	if (ThisFunction)
	{
		// If a param name was specified, check it first for UPARAM metadata
		if (!ParamName.IsNone())
		{
			FProperty* ParamProp = FindFProperty<FProperty>(ThisFunction, ParamName);
			if (ParamProp)
			{
				FilterString = GetCategoriesMetaFromField(ParamProp);
			}
		}

		// No filter found so far, fall back to UFUNCTION-level
		if (FilterString.IsEmpty())
		{
			FilterString = GetCategoriesMetaFromField(ThisFunction);
		}
	}

	return FilterString;
}

void UGameplayTagsManager::GetAllTagsFromSource(FName TagSource, TArray< TSharedPtr<FGameplayTagNode> >& OutTagArray) const
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	for (const TPair<FGameplayTag, TSharedPtr<FGameplayTagNode>>& NodePair : GameplayTagNodeMap)
	{
		if (NodePair.Value->SourceNames.Contains(TagSource))
		{
			OutTagArray.Add(NodePair.Value);
		}
	}
}

bool UGameplayTagsManager::IsDictionaryTag(FName TagName) const
{
	TSharedPtr<FGameplayTagNode> Node = FindTagNode(TagName);
	if (Node.IsValid() && Node->bIsExplicitTag)
	{
		return true;
	}

	return false;
}

bool UGameplayTagsManager::GetTagEditorData(FName TagName, FString& OutComment, FName& OutFirstTagSource, bool& bOutIsTagExplicit, bool &bOutIsRestrictedTag, bool &bOutAllowNonRestrictedChildren) const
{
	TSharedPtr<FGameplayTagNode> Node = FindTagNode(TagName);
	if (Node.IsValid())
	{
		OutComment = Node->DevComment;
		OutFirstTagSource = Node->GetFirstSourceName();
		bOutIsTagExplicit = Node->bIsExplicitTag;
		bOutIsRestrictedTag = Node->bIsRestrictedTag;
		bOutAllowNonRestrictedChildren = Node->bAllowNonRestrictedChildren;
		return true;
	}
	return false;
}

bool UGameplayTagsManager::GetTagEditorData(FName TagName, FString& OutComment, TArray<FName>& OutTagSources, bool& bOutIsTagExplicit, bool &bOutIsRestrictedTag, bool &bOutAllowNonRestrictedChildren) const
{
	TSharedPtr<FGameplayTagNode> Node = FindTagNode(TagName);
	if (Node.IsValid())
	{
		OutComment = Node->DevComment;
		OutTagSources = Node->GetAllSourceNames();
		bOutIsTagExplicit = Node->bIsExplicitTag;
		bOutIsRestrictedTag = Node->bIsRestrictedTag;
		bOutAllowNonRestrictedChildren = Node->bAllowNonRestrictedChildren;
		return true;
	}
	return false;
}

#if WITH_EDITOR

void UGameplayTagsManager::EditorRefreshGameplayTagTree()
{
	if (DeferGameplayTagTreeRebuilds > 0)
	{
		// This will call HandleGameplayTagTreeChanged(true) which comes back to here
		bGameplayTagTreeRebuildWasDeferred = true;

		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(UGameplayTagsManager::EditorRefreshGameplayTagTree)

	// Clear out source path info so it will reload off disk
	for (TPair<FString, FGameplayTagSearchPathInfo>& Pair : RegisteredSearchPaths)
	{
		Pair.Value.bWasSearched = false;
	}

	DestroyGameplayTagTree();
	LoadGameplayTagTables(false);
	ConstructGameplayTagTree();

	OnEditorRefreshGameplayTagTree.Broadcast();
}

void UGameplayTagsManager::SuspendEditorRefreshGameplayTagTree(FGuid SuspendToken)
{
	PushDeferGameplayTagTreeRebuilds();
}

void UGameplayTagsManager::ResumeEditorRefreshGameplayTagTree(FGuid SuspendToken)
{
	PopDeferGameplayTagTreeRebuilds();
}

#endif //if WITH_EDITOR

FGameplayTagContainer UGameplayTagsManager::RequestGameplayTagChildrenInDictionary(const FGameplayTag& GameplayTag) const
{
	// Note this purposefully does not include the passed in GameplayTag in the container.
	FGameplayTagContainer TagContainer;

	TSharedPtr<FGameplayTagNode> GameplayTagNode = FindTagNode(GameplayTag);
	if (GameplayTagNode.IsValid())
	{
		AddChildrenTags(TagContainer, GameplayTagNode, true, true);
	}
	return TagContainer;
}

#if WITH_EDITORONLY_DATA
FGameplayTagContainer UGameplayTagsManager::RequestGameplayTagDirectDescendantsInDictionary(const FGameplayTag& GameplayTag, EGameplayTagSelectionType SelectionType) const
{
	bool bIncludeRestrictedTags = (SelectionType == EGameplayTagSelectionType::RestrictedOnly || SelectionType == EGameplayTagSelectionType::All);
	bool bIncludeNonRestrictedTags = (SelectionType == EGameplayTagSelectionType::NonRestrictedOnly || SelectionType == EGameplayTagSelectionType::All);

	// Note this purposefully does not include the passed in GameplayTag in the container.
	FGameplayTagContainer TagContainer;

	TSharedPtr<FGameplayTagNode> GameplayTagNode = FindTagNode(GameplayTag);
	if (GameplayTagNode.IsValid())
	{
		TArray< TSharedPtr<FGameplayTagNode> >& ChildrenNodes = GameplayTagNode->GetChildTagNodes();
		int32 CurrArraySize = ChildrenNodes.Num();
		for (int32 Idx = 0; Idx < CurrArraySize; ++Idx)
		{
			TSharedPtr<FGameplayTagNode> ChildNode = ChildrenNodes[Idx];
			if (ChildNode.IsValid())
			{
				// if the tag isn't in the dictionary, add its children to the list
				if (ChildNode->GetFirstSourceName() == NAME_None)
				{
					TArray< TSharedPtr<FGameplayTagNode> >& GrandChildrenNodes = ChildNode->GetChildTagNodes();
					ChildrenNodes.Append(GrandChildrenNodes);
					CurrArraySize = ChildrenNodes.Num();
				}
				else
				{
					// this tag is in the dictionary so add it to the list
					if ((ChildNode->bIsRestrictedTag && bIncludeRestrictedTags) ||
						(!ChildNode->bIsRestrictedTag && bIncludeNonRestrictedTags))
					{
						TagContainer.AddTag(ChildNode->GetCompleteTag());
					}
				}
			}
		}
	}
	return TagContainer;
}
#endif // WITH_EDITORONLY_DATA

void UGameplayTagsManager::NotifyGameplayTagDoubleClickedEditor(FString TagName)
{
	FGameplayTag Tag = RequestGameplayTag(FName(*TagName), false);
	if(Tag.IsValid())
	{
		FSimpleMulticastDelegate Delegate;
		OnGatherGameplayTagDoubleClickedEditor.Broadcast(Tag, Delegate);
		Delegate.Broadcast();
	}
}

bool UGameplayTagsManager::ShowGameplayTagAsHyperLinkEditor(FString TagName)
{
	FGameplayTag Tag = RequestGameplayTag(FName(*TagName), false);
	if(Tag.IsValid())
	{
		FSimpleMulticastDelegate Delegate;
		OnGatherGameplayTagDoubleClickedEditor.Broadcast(Tag, Delegate);
		return Delegate.IsBound();
	}
	return false;
}

class UGameplayTagsManagerIncrementalCookFunctions
{
public:
	static void Validate(FCbFieldViewIterator Args, UE::Cook::FCookDependencyContext& Context)
	{
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		Manager.ValidateGameplayTagCookDependencies(Args, Context);
	}
};

UE_COOK_DEPENDENCY_FUNCTION(GameplayTagDependency, UGameplayTagsManagerIncrementalCookFunctions::Validate);

UE::Cook::FCookDependency UGameplayTagsManager::CreateCookDependency(const FGameplayTag& Tag) const
{
	GameplayTagUtil::FGameplayTagCookDependency Dependencies = CreateGameplayTagCookDependency(Tag);

	FCbWriter Writer;
	Dependencies.Save(Writer);
	FCbFieldIterator Fields = Writer.Save();

	return UE::Cook::FCookDependency::Function(UE_COOK_DEPENDENCY_FUNCTION_CALL(GameplayTagDependency), MoveTemp(Fields));
}

void UGameplayTagsManager::AppendDependencyHash(const FName& TagName, FBlake3& OutHasher) const
{
	FGameplayTag Tag(TagName);
	GameplayTagUtil::FGameplayTagCookDependency Dependencies = CreateGameplayTagCookDependency(Tag);
	Dependencies.Hash(OutHasher);
}

GameplayTagUtil::FGameplayTagCookDependency UGameplayTagsManager::CreateGameplayTagCookDependency(const FGameplayTag& Tag) const
{
	GameplayTagUtil::FGameplayTagCookDependency Dependencies;
	{
		UE::TScopeLock Lock(GameplayTagMapCritical);

		// Redirectors
		bool bRedirectorFound = FGameplayTagRedirectors::Get().FindRedirectorsFromTargetTagName(Tag.GetTagName(), Dependencies.SourceTagNames);

		// Does the tag exist
		const TSharedPtr<FGameplayTagNode>* Node = GameplayTagNodeMap.Find(Tag);
		Dependencies.bTagExists = (Node && Node->IsValid()) ? true : false;

		Dependencies.TagName = Tag.GetTagName();
	}

	return Dependencies;
}

void UGameplayTagsManager::ValidateGameplayTagCookDependencies(const FCbFieldViewIterator& Args, UE::Cook::FCookDependencyContext& Context) const
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	FCbFieldViewIterator ArgsField(Args);
	GameplayTagUtil::FGameplayTagCookDependency Dependency;
	bool bSuccess = Dependency.Load(ArgsField);
	if (!bSuccess)
	{
		Context.LogInvalidated(FString::Printf(TEXT("Failed to load the parameters.")));
		return;
	}

	const FGameplayTag Tag(Dependency.TagName);
	const TSharedPtr<FGameplayTagNode>* Node = GameplayTagNodeMap.Find(Tag);
	bool bTagExists = (Node && Node->IsValid()) ? true : false;
	if (bTagExists != Dependency.bTagExists)
	{
		if (bTagExists)
		{
			Context.LogInvalidated(FString::Printf(TEXT("Tag %s was missing but does exist now."), *Dependency.TagName.ToString()));
		}
		else
		{
			Context.LogInvalidated(FString::Printf(TEXT("Tag %s used to exist but is now missing."), *Dependency.TagName.ToString()));
		}

		return;
	}

	if (Dependency.SourceTagNames.IsEmpty())
	{
		// TagName was not redirected in the past. Let's check if it has a redirection now
		FGameplayTag TargetTag;
		bool bRedirectorFound = FGameplayTagRedirectors::Get().RedirectTag(Dependency.TagName, TargetTag);
		if (bRedirectorFound)
		{
			Context.LogInvalidated(FString::Printf(TEXT("New redirection for tag %s. Target tag is %s."),
				*Dependency.TagName.ToString(), *TargetTag.GetTagName().ToString()));

			return;
		}
	}
	else
	{
		// TagName was the target of redirectors. SourceTagNames used to be the sources. Let's check if the sources have changed.
		// I don't use FGameplayTagRedirectors::FindRedirectorsFromTargetTagName here because RedirectTag is faster and because I don't care
		// about any new redirectors.
		for (const FName& SourceTagName : Dependency.SourceTagNames)
		{
			FGameplayTag TargetTag;
			bool bRedirectorFound = FGameplayTagRedirectors::Get().RedirectTag(SourceTagName, TargetTag);
			if (!bRedirectorFound)
			{
				Context.LogInvalidated(FString::Printf(TEXT("Tag %s used to be redirected to %s but the redirection can't be found anymore."),
					*SourceTagName.ToString(), *Dependency.TagName.ToString()));
				return;
			}

			FName TargetTagName = TargetTag.GetTagName();
			if (TargetTagName != Dependency.TagName)
			{
				Context.LogInvalidated(FString::Printf(TEXT("Tag %s redirection was modified. It used to be redirected to %s and is now redirected to %s."),
					*SourceTagName.ToString(), *Dependency.TagName.ToString(), *TargetTagName.ToString()));

				return;
			}
		}
	}
}

void FGameplayTagNode::Hash(FBlake3& Hasher)
{
	FNameBuilder Builder;
	Builder << Tag;
	for (TCHAR& C : Builder)
	{
		C = FChar::ToLower(C);
	}
	Hasher.Update(*Builder, Builder.Len() * sizeof(**Builder));
	TArray<FName> SortedNames = SourceNames;
	Algo::Sort(SortedNames, FNameLexicalLess());
	for (FName SourceName : SortedNames)
	{
		Builder.Reset();
		Builder << SourceName;
		for (TCHAR& C : Builder)
		{
			C = FChar::ToLower(C);
		}
		Hasher.Update(*Builder, Builder.Len() * sizeof(**Builder));
	}
	Hasher.Update(*DevComment, DevComment.Len() * sizeof(**DevComment));

	uint8 Flags = 0;
	int32 BitCount = 0;
	Flags |= (bIsRestrictedTag ? 1 : 0) << BitCount++;
	Flags |= (bAllowNonRestrictedChildren ? 1 : 0) << BitCount++;
	Flags |= (bIsExplicitTag ? 1 : 0) << BitCount++;

	// These flags are transient and only used for UI display in the interactive editor
	// Flags |= (bDescendantHasConflict ? 1 : 0) << BitCount++;
	// Flags |= (bNodeHasConflict ? 1 : 0) << BitCount++;
	// Flags |= (bAncestorHasConflict ? 1 : 0) << BitCount++;
	check(BitCount <= sizeof(Flags) * 8);
	Hasher.Update(&Flags, sizeof(Flags));
}
#endif // WITH_EDITOR

const FGameplayTagSource* UGameplayTagsManager::FindTagSource(FName TagSourceName) const
{
	return TagSources.Find(TagSourceName);
}

FGameplayTagSource* UGameplayTagsManager::FindTagSource(FName TagSourceName)
{
	return TagSources.Find(TagSourceName);
}

void UGameplayTagsManager::FindTagsWithSource(FStringView PackageNameOrPath, TArray<FGameplayTag>& OutTags) const
{
	for (const auto& TagSourceEntry : TagSources)
	{
		const FGameplayTagSource& Source = TagSourceEntry.Value;
		
		FString SourcePackagePath;
		switch (Source.SourceType)
		{
		case EGameplayTagSourceType::TagList:
			if (Source.SourceTagList)
			{
				const FString ContentFilePath = FPaths::GetPath(Source.SourceTagList->ConfigFileName) / TEXT("../../Content/");
				FString RootContentPath;
				if (FPackageName::TryConvertFilenameToLongPackageName(ContentFilePath, RootContentPath))
				{
					SourcePackagePath = *RootContentPath;
				}
			}
			break;
		case EGameplayTagSourceType::DataTable:
			SourcePackagePath = Source.SourceName.ToString();
			break;
		case EGameplayTagSourceType::Native:
			SourcePackagePath = Source.SourceName.ToString();
		default:
			break;
		}

		if (SourcePackagePath.StartsWith(PackageNameOrPath))
		{
			if (Source.SourceTagList)
			{
				for (const FGameplayTagTableRow& Row : Source.SourceTagList->GameplayTagList)
				{
					OutTags.Add(FGameplayTag(Row.Tag));
				}
			}
		}
	}
}

void UGameplayTagsManager::FindTagSourcesWithType(EGameplayTagSourceType TagSourceType, TArray<const FGameplayTagSource*>& OutArray) const
{
	for (auto TagSourceIt = TagSources.CreateConstIterator(); TagSourceIt; ++TagSourceIt)
	{
		if (TagSourceIt.Value().SourceType == TagSourceType)
		{
			OutArray.Add(&TagSourceIt.Value());
		}
	}
}

FGameplayTagSource* UGameplayTagsManager::FindOrAddTagSource(FName TagSourceName, EGameplayTagSourceType SourceType, const FString& RootDirToUse)
{
	FGameplayTagSource* FoundSource = FindTagSource(TagSourceName);
	if (FoundSource)
	{
		if (SourceType == FoundSource->SourceType)
		{
			return FoundSource;
		}

		return nullptr;
	}

	// Need to make a new one

	FGameplayTagSource* NewSource = &TagSources.Add(TagSourceName, FGameplayTagSource(TagSourceName, SourceType));

	if (SourceType == EGameplayTagSourceType::Native)
	{
		NewSource->SourceTagList = NewObject<UGameplayTagsList>(this, TagSourceName, RF_Transient);
	}
	else if (SourceType == EGameplayTagSourceType::DefaultTagList)
	{
		NewSource->SourceTagList = GetMutableDefault<UGameplayTagsSettings>();
	}
	else if (SourceType == EGameplayTagSourceType::TagList)
	{
		NewSource->SourceTagList = NewObject<UGameplayTagsList>(this, TagSourceName, RF_Transient);
		if (RootDirToUse.IsEmpty())
		{
			NewSource->SourceTagList->ConfigFileName = FString::Printf(TEXT("%sTags/%s"), *FPaths::SourceConfigDir(), *TagSourceName.ToString());
		}
		else
		{
			// Use custom root and add the root to the search list for later refresh
			NewSource->SourceTagList->ConfigFileName = RootDirToUse / *TagSourceName.ToString();
			RegisteredSearchPaths.FindOrAdd(RootDirToUse);
		}
		if (GUObjectArray.IsDisregardForGC(this))
		{
			NewSource->SourceTagList->AddToRoot();
		}
	}
	else if (SourceType == EGameplayTagSourceType::RestrictedTagList)
	{
		NewSource->SourceRestrictedTagList = NewObject<URestrictedGameplayTagsList>(this, TagSourceName, RF_Transient);
		if (RootDirToUse.IsEmpty())
		{
			NewSource->SourceRestrictedTagList->ConfigFileName = FString::Printf(TEXT("%sTags/%s"), *FPaths::SourceConfigDir(), *TagSourceName.ToString());
		}
		else
		{			
			// Use custom root and add the root to the search list for later refresh
			NewSource->SourceRestrictedTagList->ConfigFileName = RootDirToUse / *TagSourceName.ToString();
			RegisteredSearchPaths.FindOrAdd(RootDirToUse);
		}
		if (GUObjectArray.IsDisregardForGC(this))
		{
			NewSource->SourceRestrictedTagList->AddToRoot();
		}
	}

	return NewSource;
}

DECLARE_CYCLE_STAT(TEXT("UGameplayTagsManager::RequestGameplayTag"), STAT_UGameplayTagsManager_RequestGameplayTag, STATGROUP_GameplayTags);

void UGameplayTagsManager::RequestGameplayTagContainer(const TArray<FString>& TagStrings, FGameplayTagContainer& OutTagsContainer, bool bErrorIfNotFound/*=true*/) const
{
	for (const FString& CurrentTagString : TagStrings)
	{
		FGameplayTag RequestedTag = RequestGameplayTag(FName(*(CurrentTagString.TrimStartAndEnd())), bErrorIfNotFound);
		if (RequestedTag.IsValid())
		{
			OutTagsContainer.AddTag(RequestedTag);
		}
	}
}

FGameplayTag UGameplayTagsManager::RequestGameplayTag(FName TagName, bool ErrorIfNotFound) const
{
	SCOPE_CYCLE_COUNTER(STAT_UGameplayTagsManager_RequestGameplayTag);

	// This critical section is to handle an issue where tag requests come from another thread when async loading from a background thread in FGameplayTagContainer::Serialize.
	// This function is not generically threadsafe.
	UE::TScopeLock Lock(GameplayTagMapCritical);

	// Check if there are redirects for this tag. If so and the redirected tag is in the node map, return it.
	// Redirects take priority, even if the tag itself may exist.
	FGameplayTag RedirectedTag;
	if (FGameplayTagRedirectors::Get().RedirectTag(TagName, RedirectedTag))
	{
		// Check if the redirected tag exists in the node map
		if (GameplayTagNodeMap.Contains(RedirectedTag))
		{
			return RedirectedTag;
		}

		// The tag that was redirected to was not found. Error if that was requested.
		if (ErrorIfNotFound)
		{
			static TSet<FName> MissingRedirectedTagNames;
			if (!MissingRedirectedTagNames.Contains(TagName))
			{
				const FString RedirectedToName = RedirectedTag.GetTagName().ToString();
				ensureAlwaysMsgf(false, TEXT("Requested Gameplay Tag %s was redirected to %s but %s was not found. Fix or remove the redirect from config."), *TagName.ToString(), *RedirectedToName, *RedirectedToName);
				MissingRedirectedTagNames.Add(TagName);
			}
		}
		
		// TagName got redirected to a non-existent tag. We'll return an empty tag rather than falling through
		// and trying to resolve the original tag name. Stale redirects should be fixed.
		return FGameplayTag();
	}

	// Check if the tag itself exists in the node map. If so, return it.
	const FGameplayTag PossibleTag(TagName);
	if (GameplayTagNodeMap.Contains(PossibleTag))
	{
		return PossibleTag;
	}

	// The tag is not found. Error if that was requested.
	if (ErrorIfNotFound)
	{
		static TSet<FName> MissingTagName;
		if (!MissingTagName.Contains(TagName))
		{
			ensureAlwaysMsgf(false, TEXT("Requested Gameplay Tag %s was not found, tags must be loaded from config or registered as a native tag"), *TagName.ToString());
			MissingTagName.Add(TagName);
		}
	}

	return FGameplayTag();
}

namespace UE::GameplayTags::Private
{

template <typename FixedStringType>
bool IsValidGameplayTagString(const FStringView& TagString, FText* OutError, FixedStringType* OutFixedString, const FString& InvalidTagCharacters)
{
	bool bIsValid = true;
	FStringView FixedString = TagString;
	TArray<FText> Errors;

	if (FixedString.IsEmpty())
	{
		Errors.Add(LOCTEXT("EmptyStringError", "Tag may not be empty"));
		bIsValid = false;
	}

	constexpr FAsciiSet Period(".");
	constexpr FAsciiSet Space(" ");
	constexpr FAsciiSet TrimSet = Period | Space;

	if (const FStringView Trimmed = FAsciiSet::TrimPrefixWith(FixedString, TrimSet); Trimmed.Len() != FixedString.Len())
	{
		if (OutError)
		{
			const FStringView Removed = FixedString.LeftChop(Trimmed.Len());
			if (FAsciiSet::HasAny(Removed, Period))
			{
				Errors.Add(LOCTEXT("StartWithPeriod", "Tag may not begin with a period ('.')"));
			}
			if (FAsciiSet::HasAny(Removed, Space))
			{
				Errors.Add(LOCTEXT("StartWithSpace", "Tag may not begin with a space"));
			}
		}
		bIsValid = false;
		FixedString = Trimmed;
	}

	if (const FStringView Trimmed = FAsciiSet::TrimSuffixWith(FixedString, TrimSet); Trimmed.Len() != FixedString.Len())
	{
		if (OutError)
		{
			const FStringView Removed = FixedString.RightChop(Trimmed.Len());
			if (FAsciiSet::HasAny(Removed, Period))
			{
				Errors.Add(LOCTEXT("EndWithPeriod", "Tag may not end with a period ('.')"));
			}
			if (FAsciiSet::HasAny(Removed, Space))
			{
				Errors.Add(LOCTEXT("EndWithSpace", "Tag may not end with a space"));
			}
		}
		bIsValid = false;
		FixedString = Trimmed;
	}

	/*
	FStringView SubString = TagString;
	while (SubString.Len() > 0)
	{
		const FStringView Trimmed = FAsciiSet::TrimPrefixWithout(SubString, Period);
		if (const FStringView SubTrim = FAsciiSet::TrimPrefixWith(Trimmed.RightChop(1), Period); SubTrim.Len() != Trimmed.Len() - 1 && SubTrim.Len() > 0)
		{
			if (OutError)
			{
				Errors.Add(LOCTEXT("DoubleDelimiter", "Tag may not have double delimiters ('..')"));
			}
			bIsValid = false;
			TArray<FStringView> FixedSubstrings = {FixedString.LeftChop(Trimmed.Len()), SubTrim};
			TStringBuilder<2048> Rebuilder(InPlace, UE::String::Join(FixedSubstrings, '.'));
			FixedString = Rebuilder.ToString();
		}

		SubString = Trimmed.RightChop(1);
	}
	*/

	FText ErrorText;
	const FText TagContext = LOCTEXT("GameplayTagContext", "Tag");
	if (!FName::IsValidXName(FixedString, InvalidTagCharacters, OutError ? &ErrorText : nullptr, &TagContext))
	{
		if (OutError)
		{
			Errors.Add(MoveTemp(ErrorText));
		}
		if (OutFixedString)
		{
			OutFixedString->Reset();
			OutFixedString->Reserve(FixedString.Len());
			for (const TCHAR& Char : FixedString)
			{
				int32 CharIndex;
				if (InvalidTagCharacters.FindChar(Char, CharIndex))
				{
					*OutFixedString += TEXT('_');
				}
				else
				{
					*OutFixedString += Char;
				}
			}
		}
		bIsValid = false;
	}
	else if (OutFixedString)
	{
		OutFixedString->Reset();
		*OutFixedString += FixedString;
	}

	if (OutError)
	{
		if (!Errors.IsEmpty())
		{
			*OutError = FText::Join(LOCTEXT("ErrorDelimiter", ", "), Errors);
		}
		else
		{
			*OutError = FText();
		}
	}

	return bIsValid;
}

} // namespace UE::GameplayTags::Private

bool UGameplayTagsManager::IsValidGameplayTagString(const TCHAR* TagString, FText* OutError /*= nullptr*/, FString* OutFixedString /*= nullptr*/)
{
	return UE::GameplayTags::Private::IsValidGameplayTagString(FStringView(TagString), OutError, OutFixedString, InvalidTagCharacters);
}

bool UGameplayTagsManager::IsValidGameplayTagString(const FString& TagString, FText* OutError /*= nullptr*/, FString* OutFixedString /*= nullptr*/)
{
	return UE::GameplayTags::Private::IsValidGameplayTagString(FStringView(TagString), OutError, OutFixedString, InvalidTagCharacters);
}

bool UGameplayTagsManager::IsValidGameplayTagString(const FStringView& TagString, FText* OutError /*= nullptr*/, FStringBuilderBase* OutFixedString /*= nullptr*/)
{
	return UE::GameplayTags::Private::IsValidGameplayTagString(FStringView(TagString), OutError, OutFixedString, InvalidTagCharacters);
}

FGameplayTag UGameplayTagsManager::FindGameplayTagFromPartialString_Slow(FString PartialString) const
{
	// This critical section is to handle an issue where tag requests come from another thread when async loading from a background thread in FGameplayTagContainer::Serialize.
	// This function is not generically threadsafe.
	UE::TScopeLock Lock(GameplayTagMapCritical);

	// Exact match first
	FGameplayTag PossibleTag(*PartialString);
	if (GameplayTagNodeMap.Contains(PossibleTag))
	{
		return PossibleTag;
	}

	// Find shortest tag name that contains the match string
	FGameplayTag FoundTag;
	FGameplayTagContainer AllTags;
	RequestAllGameplayTags(AllTags, false);

	int32 BestMatchLength = MAX_int32;
	for (FGameplayTag MatchTag : AllTags)
	{
		FString Str = MatchTag.ToString();
		if (Str.Contains(PartialString))
		{
			if (Str.Len() < BestMatchLength)
			{
				FoundTag = MatchTag;
				BestMatchLength = Str.Len();
			}
		}
	}
	
	return FoundTag;
}

FGameplayTag UGameplayTagsManager::AddNativeGameplayTag(FName TagName, const FString& TagDevComment)
{
	if (TagName.IsNone())
	{
		return FGameplayTag();
	}

	FGameplayTag NewTag = FGameplayTag(TagName);

	if (!LegacyNativeTags.Contains(TagName))
	{
		LegacyNativeTags.Add(TagName);
	}

	// Add to temporary tree and call nondestructive notify, which won't do anything if we're still in initial startup
	AddTagTableRow(FGameplayTagTableRow(TagName, TagDevComment), FGameplayTagSource::GetNativeName());

	HandleGameplayTagTreeChanged(false);

	return NewTag;
}

void UGameplayTagsManager::AddNativeGameplayTag(FNativeGameplayTag* TagSource)
{
	const FGameplayTagSource* NativeSource = FindOrAddTagSource(TagSource->GetModuleName(), EGameplayTagSourceType::Native);
	NativeSource->SourceTagList->GameplayTagList.Add(TagSource->GetGameplayTagTableRow());
	
	// This adds it to the temporary tree, but expects the caller to add it to FNativeGameplayTag::GetRegisteredNativeTags for later refreshes
	AddTagTableRow(TagSource->GetGameplayTagTableRow(), NativeSource->SourceName);

	HandleGameplayTagTreeChanged(false);
}

void UGameplayTagsManager::RemoveNativeGameplayTag(const FNativeGameplayTag* TagSource)
{
	if (!ShouldUnloadTags())
	{
		// Ignore if not allowed right now
		return;
	}

	// ~FNativeGameplayTag already removed the tag from the global list, so recreate the tree
	HandleGameplayTagTreeChanged(true);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // Remove when OnLastChanceToAddNativeTags is made private

FDelegateHandle UGameplayTagsManager::CallOrRegister_OnAddNativeTagsDelegate(const FSimpleMulticastDelegate::FDelegate& Delegate)
{
	UGameplayTagsManager* Manager = UGameplayTagsManager::GetIfAllocated();
	if (Manager && Manager->bDoneAddingNativeTags)
	{
		if (!Manager->HandleModuleObjectsLoadedHandle.IsValid())
		{
			// Need to schedule a module loaded callback as this is called from the FGameplayTagNativeAdder constructor which can't safely call the virtual functions at this scope
			Manager->HandleModuleObjectsLoadedHandle = FModuleManager::Get().OnProcessLoadedObjectsCallback().AddUObject(Manager, &UGameplayTagsManager::HandleModuleObjectsLoaded);
		}
	}

	return OnLastChanceToAddNativeTags().Add(Delegate);
}

FDelegateHandle UGameplayTagsManager::CallOrRegister_OnDoneAddingNativeTagsDelegate(const FSimpleMulticastDelegate::FDelegate& Delegate)
{
	UGameplayTagsManager* Manager = UGameplayTagsManager::GetIfAllocated();
	if (Manager && Manager->bDoneAddingNativeTags)
	{
		Delegate.Execute();
		return FDelegateHandle{};
	}
	else
	{
		return OnDoneAddingNativeTagsDelegate().Add(Delegate);
	}
}

void UGameplayTagsManager::UnregisterNativeTagDelegate(FDelegateHandle DelegateHandle)
{
	if (DelegateHandle.IsValid() && !IsEngineExitRequested())
	{
		if (!OnLastChanceToAddNativeTags().Remove(DelegateHandle))
		{
			OnDoneAddingNativeTagsDelegate().Remove(DelegateHandle);
		}
	}
}

FSimpleMulticastDelegate& UGameplayTagsManager::OnDoneAddingNativeTagsDelegate()
{
	static FSimpleMulticastDelegate Delegate;
	return Delegate;
}

FSimpleMulticastDelegate& UGameplayTagsManager::OnLastChanceToAddNativeTags()
{
	static FSimpleMulticastDelegate Delegate;
	return Delegate;
}

void UGameplayTagsManager::DoneAddingNativeTags()
{
	// Safe to call multiple times, only works the first time, must be called after the engine
	// is initialized (DoneAddingNativeTags is bound to PostEngineInit to cover anything that's skipped).
	// CallOrRegister_OnAddNativeTags will execute immediately after the first successful call
	if (GEngine && !bDoneAddingNativeTags)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UGameplayTagsManager::DoneAddingNativeTags);
		
		FSimpleMulticastDelegate& AddNativeTagsDelegate = OnLastChanceToAddNativeTags();

		UE_CLOGF(GAMEPLAYTAGS_VERBOSE, LogGameplayTags, Display, "UGameplayTagsManager::DoneAddingNativeTags. DelegateIsBound: %d", (int32)AddNativeTagsDelegate.IsBound());
		AddNativeTagsDelegate.Broadcast();
		AddNativeTagsDelegate.Clear();

		bDoneAddingNativeTags = true;

		bool bNeedsRebuild = bNeedsTreeRebuildOnDoneAddingGameplayTags;
		if (!bNeedsRebuild)
		{
			for (const TPair<FString, FGameplayTagSearchPathInfo>& Pair : RegisteredSearchPaths)
			{
				if (!Pair.Value.bWasSearched || !Pair.Value.bWasAddedToTree)
				{
					bNeedsRebuild = true;
					break;
				}
			}
		}

		if (bNeedsRebuild)
		{
			// We may add native tags that are needed for redirectors, so reconstruct the GameplayTag tree
			DestroyGameplayTagTree();
			ConstructGameplayTagTree();
		}

#if !UE_BUILD_SHIPPING
		if (FParse::Param(FCommandLine::Get(), TEXT("DumpStartupGameplayTagManagerState")))
		{
			GameplayTagUtil::DumpGameplayTagStrings(*GameplayRootTag, TEXT("GameplayTagManager/Tags.txt"));
			GameplayTagUtil::DumpRegisteredSearchPaths(RegisteredSearchPaths, TEXT("GameplayTagManager/RegisteredSearchPaths.txt"));
			GameplayTagUtil::DumpRestrictedGameplayTagSourceNames(RestrictedGameplayTagSourceNames, TEXT("GameplayTagManager/RestrictedGameplayTagSourceNames.txt"));			
		}
#endif

		OnDoneAddingNativeTagsDelegate().Broadcast();
	}
}

void UGameplayTagsManager::HandleModuleObjectsLoaded(FName InModuleName, bool bCanProcessNewlyLoadedObjects)
{
	// Remove the loaded callback
	FModuleManager::Get().OnProcessLoadedObjectsCallback().Remove(HandleModuleObjectsLoadedHandle);
	HandleModuleObjectsLoadedHandle.Reset();

	// Handle any pending native tag callbacks from the last load
	FSimpleMulticastDelegate& AddNativeTagsDelegate = OnLastChanceToAddNativeTags();
	PushDeferOnGameplayTagTreeChangedBroadcast();
	AddNativeTagsDelegate.Broadcast();
	AddNativeTagsDelegate.Clear();
	PopDeferOnGameplayTagTreeChangedBroadcast();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

FGameplayTagContainer UGameplayTagsManager::RequestGameplayTagParents(const FGameplayTag& GameplayTag) const
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	const FGameplayTagContainer* ParentTags = GetSingleTagContainerPtr(GameplayTag);

	if (ParentTags)
	{
		return ParentTags->GetGameplayTagParents();
	}
	return FGameplayTagContainer();
}

// If true, verify that the node lookup and manual methods give identical results
#define VALIDATE_EXTRACT_PARENT_TAGS 0

bool UGameplayTagsManager::ExtractParentTags(const FGameplayTag& GameplayTag, TArray<FGameplayTag>& UniqueParentTags) const
{
	// This gets called during GameplayTagContainer serialization so needs to be efficient
	if (!GameplayTag.IsValid())
	{
		return false;
	}

	TArray<FGameplayTag> ValidationCopy;

	if constexpr (0 != VALIDATE_EXTRACT_PARENT_TAGS)
	{
		ValidationCopy = UniqueParentTags;
	}

	int32 OldSize = UniqueParentTags.Num();
	FName RawTag = GameplayTag.GetTagName();

	UE::TScopeLock Lock(GameplayTagMapCritical);

	// This code does not check redirectors because that was already handled by GameplayTagContainerLoaded
	const TSharedPtr<FGameplayTagNode>*Node = GameplayTagNodeMap.Find(GameplayTag);
	if (Node)
	{
		// Use the registered tag container if it exists
		const FGameplayTagContainer& SingleContainer = (*Node)->GetSingleTagContainer();
		for (const FGameplayTag& ParentTag : SingleContainer.ParentTags)
		{
			UniqueParentTags.AddUnique(ParentTag);
		}

		if constexpr (0 != VALIDATE_EXTRACT_PARENT_TAGS)
		{
			GameplayTag.ParseParentTags(ValidationCopy);
			ensureAlwaysMsgf(ValidationCopy == UniqueParentTags, TEXT("ExtractParentTags results are inconsistent for tag %s"), *GameplayTag.ToString());
		}
	}
	else
	{
		// If we don't clear invalid tags, we need to extract the parents now in case they get registered later
		GameplayTag.ParseParentTags(UniqueParentTags);
	}

	return UniqueParentTags.Num() != OldSize;
}

void UGameplayTagsManager::RequestAllGameplayTags(FGameplayTagContainer& TagContainer, bool OnlyIncludeDictionaryTags) const
{
	UE::TScopeLock Lock(GameplayTagMapCritical);

	for (const TPair<FGameplayTag, TSharedPtr<FGameplayTagNode>>& NodePair : GameplayTagNodeMap)
	{
		const TSharedPtr<FGameplayTagNode>& TagNode = NodePair.Value;
		if (!OnlyIncludeDictionaryTags || TagNode->IsExplicitTag())
		{
			TagContainer.AddTagFast(TagNode->GetCompleteTag());
		}
	}
}

FGameplayTagContainer UGameplayTagsManager::RequestGameplayTagChildren(const FGameplayTag& GameplayTag) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameplayTagsManager_RequestGameplayTagChildren);
	UE::TScopeLock Lock(GameplayTagMapCritical);

	FGameplayTagContainer TagContainer;
	// Note this purposefully does not include the passed in GameplayTag in the container.
	TSharedPtr<FGameplayTagNode> GameplayTagNode = FindTagNode(GameplayTag);
	if (GameplayTagNode.IsValid())
	{
		if (FGameplayTagContainer* Cached = ChildNodesCache.FindAndTouch(GameplayTag))
		{
			return *Cached;
		}

		AddChildrenTags(TagContainer, GameplayTagNode, true, false);

		ChildNodesCache.SetMaxNumElements(CVarGameplayTagsLruCacheSize.GetValueOnAnyThread());
		if (ChildNodesCache.Max() > 0)
		{
			ChildNodesCache.Add(GameplayTag, TagContainer);
		}
	}
	return TagContainer;
}

FGameplayTag UGameplayTagsManager::RequestGameplayTagDirectParent(const FGameplayTag& GameplayTag) const
{
	TSharedPtr<FGameplayTagNode> GameplayTagNode = FindTagNode(GameplayTag);
	if (GameplayTagNode.IsValid())
	{
		TSharedPtr<FGameplayTagNode> Parent = GameplayTagNode->GetParentTagNode();
		if (Parent.IsValid())
		{
			return Parent->GetCompleteTag();
		}
	}
	return FGameplayTag();
}

void UGameplayTagsManager::AddChildrenTags(FGameplayTagContainer& TagContainer, TSharedPtr<FGameplayTagNode> GameplayTagNode, bool RecurseAll, bool OnlyIncludeDictionaryTags) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGameplayTagsManager_AddChildrenTags);

	TArray<FGameplayTag> TagsToAdd;
	TArray<TSharedPtr<FGameplayTagNode>> Queue;
	Queue.Push(GameplayTagNode);
	bool bFirstOne = true;
	while (!Queue.IsEmpty())
	{
		TSharedPtr<FGameplayTagNode> CurrentNode = Queue.Pop(EAllowShrinking::No);
		if (CurrentNode.IsValid())
		{
			bool bShouldInclude = !bFirstOne;

#if WITH_EDITORONLY_DATA
			if (OnlyIncludeDictionaryTags && !CurrentNode->IsExplicitTag())
			{
				// Only have info to do this in editor builds
				bShouldInclude = false;
			}
#endif

			if (bShouldInclude)
			{
				TagsToAdd.Add(CurrentNode->GetCompleteTag());
			}

			if (bFirstOne || RecurseAll)
			{
				Queue.Append(CurrentNode->GetChildTagNodes());
			}

			bFirstOne = false;
		}
	}
	TagContainer.AppendTags(FGameplayTagContainer::CreateFromArray(TagsToAdd));
}

void UGameplayTagsManager::SplitGameplayTagFName(const FGameplayTag& Tag, TArray<FName>& OutNames) const
{
	TSharedPtr<FGameplayTagNode> CurNode = FindTagNode(Tag);
	while (CurNode.IsValid())
	{
		OutNames.Insert(CurNode->GetSimpleTagName(), 0);
		CurNode = CurNode->GetParentTagNode();
	}
}

int32 UGameplayTagsManager::GameplayTagsMatchDepth(const FGameplayTag& GameplayTagOne, const FGameplayTag& GameplayTagTwo) const
{
	using FTagsArray = TArray<FName, TInlineAllocator<32>>;

	auto GetTags = [this](FTagsArray& Tags, const FGameplayTag& GameplayTag)
	{
		for (TSharedPtr<FGameplayTagNode> TagNode = FindTagNode(GameplayTag); TagNode.IsValid(); TagNode = TagNode->GetParentTagNode())
		{
			Tags.Add(TagNode->Tag);
		}
	};

	FTagsArray Tags1, Tags2;
	GetTags(Tags1, GameplayTagOne);
	GetTags(Tags2, GameplayTagTwo);

	// Get Tags returns tail to head, so compare in reverse order
	int32 Index1 = Tags1.Num() - 1;
	int32 Index2 = Tags2.Num() - 1;

	int32 Depth = 0;

	for (; Index1 >= 0 && Index2 >= 0; --Index1, --Index2)
	{
		if (Tags1[Index1] == Tags2[Index2])
		{
			++Depth;
		}
		else
		{
			break;
		}
	}

	return Depth;
}

int32 UGameplayTagsManager::GetNumberOfTagNodes(const FGameplayTag& GameplayTag) const
{
	int32 Count = 0;

	TSharedPtr<FGameplayTagNode> TagNode = FindTagNode(GameplayTag);
	while (TagNode.IsValid())
	{
		++Count;								// Increment the count of valid tag nodes.
		TagNode = TagNode->GetParentTagNode();	// Continue up the chain of parents.
	}

	return Count;
}

DECLARE_CYCLE_STAT(TEXT("UGameplayTagsManager::GetAllParentNodeNames"), STAT_UGameplayTagsManager_GetAllParentNodeNames, STATGROUP_GameplayTags);

void UGameplayTagsManager::GetAllParentNodeNames(TSet<FName>& NamesList, TSharedPtr<FGameplayTagNode> GameplayTag) const
{
	SCOPE_CYCLE_COUNTER(STAT_UGameplayTagsManager_GetAllParentNodeNames);

	NamesList.Add(GameplayTag->GetCompleteTagName());
	TSharedPtr<FGameplayTagNode> Parent = GameplayTag->GetParentTagNode();
	if (Parent.IsValid())
	{
		GetAllParentNodeNames(NamesList, Parent);
	}
}

DECLARE_CYCLE_STAT(TEXT("UGameplayTagsManager::ValidateTagCreation"), STAT_UGameplayTagsManager_ValidateTagCreation, STATGROUP_GameplayTags);

bool UGameplayTagsManager::ValidateTagCreation(FName TagName) const
{
	SCOPE_CYCLE_COUNTER(STAT_UGameplayTagsManager_ValidateTagCreation);

	return FindTagNode(TagName).IsValid();
}

#if WITH_EDITOR
void UGameplayTagsManager::DumpSources(FOutputDevice& Out) const
{
	for (const TPair<FName, FGameplayTagSource>& Pair : TagSources)
	{
		Out.Logf(TEXT("%s : %s"), *Pair.Key.ToString(), *UEnum::GetValueAsString(Pair.Value.SourceType));
		FString ConfigFilePath = Pair.Value.GetConfigFileName();
		if (!ConfigFilePath.IsEmpty())
		{
			Out.Logf(TEXT("Config file path: %s"), *Pair.Value.SourceTagList->ConfigFileName);
		}
	}
}
#endif

FGameplayTagTableRow::FGameplayTagTableRow(FGameplayTagTableRow const& Other)
{
	*this = Other;
}

FGameplayTagTableRow& FGameplayTagTableRow::operator=(FGameplayTagTableRow const& Other)
{
	// Guard against self-assignment
	if (this == &Other)
	{
		return *this;
	}

	Tag = Other.Tag;
	DevComment = Other.DevComment;

	return *this;
}

bool FGameplayTagTableRow::operator==(FGameplayTagTableRow const& Other) const
{
	return (Tag == Other.Tag);
}

bool FGameplayTagTableRow::operator!=(FGameplayTagTableRow const& Other) const
{
	return (Tag != Other.Tag);
}

bool FGameplayTagTableRow::operator<(FGameplayTagTableRow const& Other) const
{
	return UE::ComparisonUtility::CompareWithNumericSuffix(Tag, Other.Tag) < 0;
}

FRestrictedGameplayTagTableRow::FRestrictedGameplayTagTableRow(FRestrictedGameplayTagTableRow const& Other)
{
	*this = Other;
}

FRestrictedGameplayTagTableRow& FRestrictedGameplayTagTableRow::operator=(FRestrictedGameplayTagTableRow const& Other)
{
	// Guard against self-assignment
	if (this == &Other)
	{
		return *this;
	}

	Super::operator=(Other);
	bAllowNonRestrictedChildren = Other.bAllowNonRestrictedChildren;

	return *this;
}

bool FRestrictedGameplayTagTableRow::operator==(FRestrictedGameplayTagTableRow const& Other) const
{
	if (bAllowNonRestrictedChildren != Other.bAllowNonRestrictedChildren)
	{
		return false;
	}

	if (Tag != Other.Tag)
	{
		return false;
	}

	return true;
}

bool FRestrictedGameplayTagTableRow::operator!=(FRestrictedGameplayTagTableRow const& Other) const
{
	if (bAllowNonRestrictedChildren == Other.bAllowNonRestrictedChildren)
	{
		return false;
	}

	if (Tag == Other.Tag)
	{
		return false;
	}

	return true;
}

FGameplayTagNode::FGameplayTagNode(FName InTag, FName InFullTag, TSharedPtr<FGameplayTagNode> InParentNode, bool InIsExplicitTag, bool InIsRestrictedTag, bool InAllowNonRestrictedChildren)
	: Tag(InTag)
	, ParentNode(InParentNode)
	, NetIndex(INVALID_TAGNETINDEX)
{
	// Manually construct the tag container as we want to bypass the safety checks
	CompleteTagWithParents.GameplayTags.Add(FGameplayTag(InFullTag));

	FGameplayTagNode* RawParentNode = ParentNode.Get();
	if (RawParentNode && RawParentNode->GetSimpleTagName() != NAME_None)
	{
		// Our parent nodes are already constructed, and must have it's tag in GameplayTags[0]
		const FGameplayTagContainer ParentContainer = RawParentNode->GetSingleTagContainer();

		CompleteTagWithParents.ParentTags.Add(ParentContainer.GameplayTags[0]);
		CompleteTagWithParents.ParentTags.Append(ParentContainer.ParentTags);
	}
	
#if WITH_EDITORONLY_DATA
	bIsExplicitTag = InIsExplicitTag;
	bIsRestrictedTag = InIsRestrictedTag;
	bAllowNonRestrictedChildren = InAllowNonRestrictedChildren;

	bDescendantHasConflict = false;
	bNodeHasConflict = false;
	bAncestorHasConflict = false;
#endif 
}

void FGameplayTagNode::ResetNode()
{
	Tag = NAME_None;
	CompleteTagWithParents.Reset();
	NetIndex = INVALID_TAGNETINDEX;

	for (int32 ChildIdx = 0; ChildIdx < ChildTags.Num(); ++ChildIdx)
	{
		ChildTags[ChildIdx]->ResetNode();
	}

	ChildTags.Empty();
	ParentNode.Reset();

#if WITH_EDITORONLY_DATA
	SourceNames.Reset();
	DevComment = "";
	bIsExplicitTag = false;
	bIsRestrictedTag = false;
	bAllowNonRestrictedChildren = false;
	bDescendantHasConflict = false;
	bNodeHasConflict = false;
	bAncestorHasConflict = false;
#endif 
}

#undef LOCTEXT_NAMESPACE
