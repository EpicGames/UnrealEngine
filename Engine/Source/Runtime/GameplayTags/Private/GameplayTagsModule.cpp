// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsModule.h"
#include "Misc/ConfigCacheIni.h"
#include "HAL/IConsoleManager.h"
#include "UObject/Package.h"

FSimpleMulticastDelegate IGameplayTagsModule::OnGameplayTagTreeChanged;
FSimpleMulticastDelegate IGameplayTagsModule::OnTagSettingsChanged;

class FGameplayTagsModule : public IGameplayTagsModule
{
	// Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface
};

IMPLEMENT_MODULE( FGameplayTagsModule, GameplayTags )
DEFINE_LOG_CATEGORY(LogGameplayTags);

void FGameplayTagsModule::StartupModule()
{
	// This will force initialization
	UGameplayTagsManager::Get();
}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
int32 GameplayTagPrintReportOnShutdown = 0;
static FAutoConsoleVariableRef CVarGameplayTagPrintReportOnShutdown(TEXT("GameplayTags.PrintReportOnShutdown"), GameplayTagPrintReportOnShutdown, TEXT("Print gameplay tag replication report on shutdown"), ECVF_Default );
#endif


void FGameplayTagsModule::ShutdownModule()
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	if (GameplayTagPrintReportOnShutdown)
	{
		UGameplayTagsManager::Get().PrintReplicationFrequencyReport();
	}
#endif

	UGameplayTagsManager::SingletonManager = nullptr;
}
