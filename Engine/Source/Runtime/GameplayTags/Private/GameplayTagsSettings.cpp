// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsSettings.h"
#include "GameplayTagsModule.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsSettings)

UGameplayTagsList::UGameplayTagsList(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// No config filename, needs to be set at creation time
}

void UGameplayTagsList::SortTags()
{
	GameplayTagList.Sort();
}

URestrictedGameplayTagsList::URestrictedGameplayTagsList(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// No config filename, needs to be set at creation time
}

void URestrictedGameplayTagsList::SortTags()
{
	RestrictedGameplayTagList.Sort();
}

bool FRestrictedConfigInfo::operator==(const FRestrictedConfigInfo& Other) const
{
	if (RestrictedConfigName != Other.RestrictedConfigName)
	{
		return false;
	}

	if (Owners.Num() != Other.Owners.Num())
	{
		return false;
	}

	for (int32 Idx = 0; Idx < Owners.Num(); ++Idx)
	{
		if (Owners[Idx] != Other.Owners[Idx])
		{
			return false;
		}
	}

	return true;
}

bool FRestrictedConfigInfo::operator!=(const FRestrictedConfigInfo& Other) const
{
	return !(operator==(Other));
}

UGameplayTagsSettings::UGameplayTagsSettings(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	ConfigFileName = GetDefaultConfigFilename();
	ImportTagsFromConfig = true;
	WarnOnInvalidTags = true;
	FastReplication = false;
	bDynamicReplication = false; // This currently needs Iris enabled (so don't enable by default)
	AllowEditorTagUnloading = true;
	AllowGameTagUnloading = false;
	InvalidTagCharacters = ("\"',");
	NumBitsForContainerSize = 6;
	NetIndexFirstBitSegment = 16;
}

#if WITH_EDITOR
void UGameplayTagsSettings::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	if (PropertyThatWillChange && PropertyThatWillChange->GetFName() == GET_MEMBER_NAME_CHECKED(UGameplayTagsSettings, RestrictedConfigFiles))
	{
		RestrictedConfigFilesTempCopy = RestrictedConfigFiles;
	}
}

void UGameplayTagsSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetName() == "RestrictedConfigName")
		{
			UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
			for (FRestrictedConfigInfo& Info : RestrictedConfigFiles)
			{
				if (!Info.RestrictedConfigName.IsEmpty())
				{
					if (!Info.RestrictedConfigName.EndsWith(TEXT(".ini")))
					{
						Info.RestrictedConfigName.Append(TEXT(".ini"));
					}
					FGameplayTagSource* Source = Manager.FindOrAddTagSource(*Info.RestrictedConfigName, EGameplayTagSourceType::RestrictedTagList);
					if (!Source)
					{
						FNotificationInfo NotificationInfo(FText::Format(NSLOCTEXT("GameplayTagsSettings", "UnableToAddRestrictedTagSource", "Unable to add restricted tag source {0}. It may already be in use."), FText::FromString(Info.RestrictedConfigName)));
						FSlateNotificationManager::Get().AddNotification(NotificationInfo);
						Info.RestrictedConfigName.Empty();
					}
				}
			}
		}

		// if we're adding a new restricted config file we will try to auto populate the owner
		if (PropertyChangedEvent.ChangeType == EPropertyChangeType::ArrayAdd && PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UGameplayTagsSettings, RestrictedConfigFiles))
		{
			if (RestrictedConfigFilesTempCopy.Num() + 1 == RestrictedConfigFiles.Num())
			{
				int32 FoundIdx = RestrictedConfigFilesTempCopy.Num();
				for (int32 Idx = 0; Idx < RestrictedConfigFilesTempCopy.Num(); ++Idx)
				{
					if (RestrictedConfigFilesTempCopy[Idx] != RestrictedConfigFiles[Idx])
					{
						FoundIdx = Idx;
						break;
					}
				}

				ensure(FoundIdx < RestrictedConfigFiles.Num());
				RestrictedConfigFiles[FoundIdx].Owners.Add(FPlatformProcess::UserName());

			}
		}
		IGameplayTagsModule::OnTagSettingsChanged.Broadcast();
	}
}
#endif

// ---------------------------------

UGameplayTagsDeveloperSettings::UGameplayTagsDeveloperSettings(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	
}

static const FName NAME_Advanced("Advanced");

FName UGameplayTagsDeveloperSettings::GetCategoryName() const
{
	return NAME_Advanced;
}

