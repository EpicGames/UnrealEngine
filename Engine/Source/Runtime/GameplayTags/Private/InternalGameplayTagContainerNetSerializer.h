// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "GameplayTagContainer.h"
#include "InternalGameplayTagContainerNetSerializer.generated.h"

USTRUCT()
struct FGameplayTagContainerNetSerializerSerializationHelper
{
	GENERATED_BODY()

	UPROPERTY();
	TArray<FGameplayTag> GameplayTags;
};
