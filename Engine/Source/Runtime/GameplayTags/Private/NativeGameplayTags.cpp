// Copyright Epic Games, Inc. All Rights Reserved.

#include "NativeGameplayTags.h"
#include "Interfaces/IProjectManager.h"
#include "ModuleDescriptor.h"
#include "ProjectDescriptor.h"
#include "PluginDescriptor.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/PackageName.h"

#define LOCTEXT_NAMESPACE "FNativeGameplayTag"

#if !UE_BUILD_SHIPPING

//FEditorDelegates

static bool VerifyModuleCanContainGameplayTag(FName ModuleName, FName TagName, const FModuleDescriptor* Module, TSharedPtr<IPlugin> OptionalOwnerPlugin)
{
	if (Module)
	{
		if (!(Module->Type == EHostType::Runtime || Module->Type == EHostType::RuntimeAndProgram))
		{
			// TODO NDarnell - If it's not a module we load always we need to make sure the tag is available in some other fashion
			// such as through an ini.

			//TSharedPtr<IPlugin> ThisPlugin = IPluginManager::Get().FindPlugin(TEXT("CommonDialogue"));
			//check(ThisPlugin.IsValid());
			//UGameplayTagsManager::Get().AddTagIniSearchPath(ThisPlugin->GetBaseDir() / TEXT("Config") / TEXT("Tags"));

			//const FString PluginFolder = FPaths::GetPath(StateProperties.PluginInstalledFilename);
			//UGameplayTagsManager::Get().AddTagIniSearchPath(PluginFolder / TEXT("Config") / TEXT("Tags"));


			//const FString& PluginDescriptorFilename = Plugin->GetDescriptorFileName();

			//// Make sure you are in the game feature plugins folder. All GameFeaturePlugins are in this folder.
			////@TODO: GameFeaturePluginEnginePush: Comments elsewhere allow plugins outside of the folder as long as they explicitly opt in, either those are wrong or this check is wrong
			//if (!PluginDescriptorFilename.IsEmpty() && FPaths::ConvertRelativePathToFull(PluginDescriptorFilename).StartsWith(GetDefault<UGameFeaturesSubsystemSettings>()->BuiltInGameFeaturePluginsFolder) && FPaths::FileExists(PluginDescriptorFilename))
			//{

			ensureAlwaysMsgf(false, TEXT("Native Gameplay Tag '%s' defined in '%s'.  The module type is '%s' but needs to be 'Runtime' or 'RuntimeAndProgram'.  Client and Server tags must match."), *TagName.ToString(), *ModuleName.ToString(), EHostType::ToString(Module->Type));
		}

		// Not a mistake - we return true even if it fails the test, the return value is a validation we were able to verify that it could or could not.
		return true;
	}

	return false;
}

#endif

FNativeGameplayTag::FNativeGameplayTag(FName InPluginName, FName InModuleName, FName TagName, const FString& TagDevComment, ENativeGameplayTagToken)
{
	// TODO NDarnell To try and make sure nobody is using these during non-static init
	// of the module, we could add an indicator on the module manager indicating
	// if we're actively loading model and make sure we only run this code during
	// that point.

#if !UE_BUILD_SHIPPING
	PluginName = InPluginName;
	ModuleName = InModuleName;
	ModulePackageName = FPackageName::GetModuleScriptPackageName(InModuleName);
#endif

	InternalTag = TagName.IsNone() ? FGameplayTag() : FGameplayTag(TagName);
#if WITH_EDITOR
	DeveloperComment = TagDevComment;
#endif

	GetRegisteredNativeTags().Add(this);

	if (UGameplayTagsManager* Manager = UGameplayTagsManager::GetIfAllocated())
	{
		Manager->AddNativeGameplayTag(this);
	}
}

FNativeGameplayTag::~FNativeGameplayTag()
{
	GetRegisteredNativeTags().Remove(this);

	if (UGameplayTagsManager* Manager = UGameplayTagsManager::GetIfAllocated())
	{
		Manager->RemoveNativeGameplayTag(this);
	}
}

FName FNativeGameplayTag::NAME_NativeGameplayTag("Native");

#if !UE_BUILD_SHIPPING

void FNativeGameplayTag::ValidateTagRegistration() const
{
	if (bValidated)
	{
		return;
	}

	bValidated = true;

	// Running commandlets or programs won't have projects potentially, so we can't assume there's a project.
	if (const FProjectDescriptor* const CurrentProject = IProjectManager::Get().GetCurrentProject())
	{
		const FModuleDescriptor* ProjectModule =
			CurrentProject->Modules.FindByPredicate([this](const FModuleDescriptor& Module) { return Module.Name == ModuleName; });

		if (!VerifyModuleCanContainGameplayTag(ModuleName, InternalTag.GetTagName(), ProjectModule, TSharedPtr<IPlugin>()))
		{
			const FModuleDescriptor* PluginModule = nullptr;

			// Ok, so we're not in a module for the project, 
			TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName.ToString());
			if (Plugin.IsValid())
			{
				const FPluginDescriptor& PluginDescriptor = Plugin->GetDescriptor();
				PluginModule = PluginDescriptor.Modules.FindByPredicate([this](const FModuleDescriptor& Module) { return Module.Name == ModuleName; });
			}

			if (!VerifyModuleCanContainGameplayTag(ModuleName, InternalTag.GetTagName(), PluginModule, Plugin))
			{
				ensureAlwaysMsgf(false, TEXT("Unable to find information about module '%s' in plugin '%s'"), *ModuleName.ToString(), *PluginName.ToString());
			}
		}
	}
}

#endif

TSet<const FNativeGameplayTag*>& FNativeGameplayTag::GetRegisteredNativeTags()
{
	static TSet<const class FNativeGameplayTag*> RegisteredNativeTags;
	return RegisteredNativeTags;
}

#undef LOCTEXT_NAMESPACE
