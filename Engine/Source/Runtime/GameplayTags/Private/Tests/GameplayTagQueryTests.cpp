// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "GameplayTagContainer.h"
#include "Misc/AutomationTest.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGameplayTagQueryTest_Empty, "System.GameplayTags.GameplayTagQuery.Empty", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

#include "GameplayTagsManager.h"

bool FGameplayTagQueryTest_Empty::RunTest(const FString& Parameters)
{
	FGameplayTagContainer AllGameplayTags;
	UGameplayTagsManager::Get().RequestAllGameplayTags(AllGameplayTags, true);
	if ( AllGameplayTags.IsEmpty() )
	{
		AddInfo(TEXT("GameplayTags are not defined for this Project. Skipping test."));
		return true;
	}

	FGameplayTag RandomTag = AllGameplayTags.First();
	FGameplayTagContainer EmptyTagContainer;
	FGameplayTagContainer NonEmptyContainer{ RandomTag };

	// The Query is Completely Empty
	{
		FGameplayTagQuery Query;
		TestFalse("Match Empty Query w/ Empty Container", Query.Matches(EmptyTagContainer));
		TestFalse("Match Empty Query w/ Non-Empty Container", Query.Matches(NonEmptyContainer));
	}

	// The Query has Expressions, but those Expressions have no valid tags
	{
		FGameplayTagQuery Query;
		FGameplayTagQueryExpression TempExpression;

		TempExpression.AllExprMatch();
		Query.Build(TempExpression, TEXT("Empty Tag Query Expression - AllExprMatch"));
		TestTrue(TEXT("Match Empty AllExprMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Empty AllExprMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.AllTagsMatch();
		Query.Build(TempExpression, TEXT("Empty Tag Query Expression - AllTagsMatch"));
		TestTrue(TEXT("Match Empty AllTagsMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Empty AllTagsMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.AnyExprMatch();
		Query.Build(TempExpression, TEXT("Empty Tag Query Expression - AnyExprMatch"));
		TestFalse(TEXT("Match Empty AnyExprMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestFalse(TEXT("Match Empty AnyExprMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.AnyTagsMatch();
		Query.Build(TempExpression, TEXT("Empty Tag Query Expression - AnyTagsMatch"));
		TestFalse(TEXT("Match Empty AnyTagsMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestFalse(TEXT("Match Empty AnyTagsMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.NoExprMatch();
		Query.Build(TempExpression, TEXT("Empty Tag Query Expression - NoExprMatch"));
		TestTrue(TEXT("Match Empty NoExprMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Empty NoExprMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.NoTagsMatch();
		Query.Build(TempExpression, TEXT("Empty Tag Query Expression - NoTagsMatch"));
		TestTrue(TEXT("Match Empty NoTagsMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Empty NoTagsMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));
	}

	// The Query has Expressions, and those Expressions have valid tags
	{
		FGameplayTagQuery Query;
		FGameplayTagQueryExpression TempExpression;
		TempExpression.TagSet = TArray<FGameplayTag>{ RandomTag };

		TempExpression.AllExprMatch();
		Query.Build(TempExpression, TEXT("Non-Empty Tag Query Expression - AllExprMatch"));
		TestTrue(TEXT("Match Non-Empty AllExprMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Non-Empty AllExprMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.AllTagsMatch();
		Query.Build(TempExpression, TEXT("Non-Empty Tag Query Expression - AllTagsMatch"));
		TestFalse(TEXT("Match Non-Empty AllTagsMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Non-Empty AllTagsMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.AnyExprMatch();
		Query.Build(TempExpression, TEXT("Non-Empty Tag Query Expression - AnyExprMatch"));
		TestFalse(TEXT("Match Non-Empty AnyExprMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestFalse(TEXT("Match Non-Empty AnyExprMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.AnyTagsMatch();
		Query.Build(TempExpression, TEXT("Non-Empty Tag Query Expression - AnyTagsMatch"));
		TestFalse(TEXT("Match Non-Empty AnyTagsMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Non-Empty AnyTagsMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.NoExprMatch();
		Query.Build(TempExpression, TEXT("Non-Empty Tag Query Expression - NoExprMatch"));
		TestTrue(TEXT("Match Non-Empty NoExprMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestTrue(TEXT("Match Non-Empty NoExprMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));

		TempExpression.NoTagsMatch();
		Query.Build(TempExpression, TEXT("Non-Empty Tag Query Expression - NoTagsMatch"));
		TestTrue(TEXT("Match Non-Empty NoTagsMatch w/ Empty Container"), Query.Matches(EmptyTagContainer));
		TestFalse(TEXT("Match Non-Empty NoTagsMatch w/ Non-Empty Container"), Query.Matches(NonEmptyContainer));
	}

	return true;
}

