// Copyright Epic Games, Inc. All Rights Reserved.
#if WITH_TESTS
#include "UObject/Package.h"
#include "Misc/AutomationTest.h"
#include "Engine/DataTable.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "GameplayTagsModule.h"
#include "Stats/StatsMisc.h"
#include "Tests/TestHarnessAdapter.h"
#if WITH_LOW_LEVEL_TESTS
#include "TestCommon/Expectations.h"
#endif

#if WITH_AUTOMATION_WORKER

class FGameplayTagTestBase : public FAutomationTestBase
{
public:
	FGameplayTagTestBase(const FString& InName, const bool bInComplexTask)
		: FAutomationTestBase(InName, bInComplexTask)
	{
	}

	UDataTable* CreateGameplayDataTable()
	{	
		constexpr FStringView TestTags[] =
		{
			TEXTVIEW("Effect.Damage"),
			TEXTVIEW("Effect.Damage.Basic"),
			TEXTVIEW("Effect.Damage.Type1"),
			TEXTVIEW("Effect.Damage.Type2"),
			TEXTVIEW("Effect.Damage.Reduce"),
			TEXTVIEW("Effect.Damage.Buffable"),
			TEXTVIEW("Effect.Damage.Buff"),
			TEXTVIEW("Effect.Damage.Physical"),
			TEXTVIEW("Effect.Damage.Fire"),
			TEXTVIEW("Effect.Damage.Buffed.FireBuff"),
			TEXTVIEW("Effect.Damage.Mitigated.Armor"),
			TEXTVIEW("Effect.Lifesteal"),
			TEXTVIEW("Effect.Shield"),
			TEXTVIEW("Effect.Buff"),
			TEXTVIEW("Effect.Immune"),
			TEXTVIEW("Effect.FireDamage"),
			TEXTVIEW("Effect.Shield.Absorb"),
			TEXTVIEW("Effect.Protect.Damage"),
			TEXTVIEW("Stackable"),
			TEXTVIEW("Stack.DiminishingReturns"),
			TEXTVIEW("GameplayCue.Burning"),
			TEXTVIEW("Expensive.Status.Tag.Type.1"),
			TEXTVIEW("Expensive.Status.Tag.Type.2"),
			TEXTVIEW("Expensive.Status.Tag.Type.3"),
			TEXTVIEW("Expensive.Status.Tag.Type.4"),
			TEXTVIEW("Expensive.Status.Tag.Type.5"),
			TEXTVIEW("Expensive.Status.Tag.Type.6"),
			TEXTVIEW("Expensive.Status.Tag.Type.7"),
			TEXTVIEW("Expensive.Status.Tag.Type.8"),
			TEXTVIEW("Expensive.Status.Tag.Type.9"),
			TEXTVIEW("Expensive.Status.Tag.Type.10"),
			TEXTVIEW("Expensive.Status.Tag.Type.11"),
			TEXTVIEW("Expensive.Status.Tag.Type.12"),
			TEXTVIEW("Expensive.Status.Tag.Type.13"),
			TEXTVIEW("Expensive.Status.Tag.Type.14"),
			TEXTVIEW("Expensive.Status.Tag.Type.15"),
			TEXTVIEW("Expensive.Status.Tag.Type.16"),
			TEXTVIEW("Expensive.Status.Tag.Type.17"),
			TEXTVIEW("Expensive.Status.Tag.Type.18"),
			TEXTVIEW("Expensive.Status.Tag.Type.19"),
			TEXTVIEW("Expensive.Status.Tag.Type.20"),
			TEXTVIEW("Expensive.Status.Tag.Type.21"),
			TEXTVIEW("Expensive.Status.Tag.Type.22"),
			TEXTVIEW("Expensive.Status.Tag.Type.23"),
			TEXTVIEW("Expensive.Status.Tag.Type.24"),
			TEXTVIEW("Expensive.Status.Tag.Type.25"),
			TEXTVIEW("Expensive.Status.Tag.Type.26"),
			TEXTVIEW("Expensive.Status.Tag.Type.27"),
			TEXTVIEW("Expensive.Status.Tag.Type.28"),
			TEXTVIEW("Expensive.Status.Tag.Type.29"),
			TEXTVIEW("Expensive.Status.Tag.Type.30"),
			TEXTVIEW("Expensive.Status.Tag.Type.31"),
			TEXTVIEW("Expensive.Status.Tag.Type.32"),
			TEXTVIEW("Expensive.Status.Tag.Type.33"),
			TEXTVIEW("Expensive.Status.Tag.Type.34"),
			TEXTVIEW("Expensive.Status.Tag.Type.35"),
			TEXTVIEW("Expensive.Status.Tag.Type.36"),
			TEXTVIEW("Expensive.Status.Tag.Type.37"),
			TEXTVIEW("Expensive.Status.Tag.Type.38"),
			TEXTVIEW("Expensive.Status.Tag.Type.39"),
			TEXTVIEW("Expensive.Status.Tag.Type.40"),
		};

		auto DataTable = NewObject<UDataTable>(GetTransientPackage(), FName("TempDataTable"));
		DataTable->RowStruct = FGameplayTagTableRow::StaticStruct();


		FString CSV(",Tag,CategoryText,");
		for (int32 Count = 0; Count < UE_ARRAY_COUNT(TestTags); Count++)
		{
			CSV.Appendf(TEXT("\r\n%d,%.*s"), Count, TestTags[Count].Len(), TestTags[Count].GetData());
		}

		DataTable->CreateTableFromCSVString(CSV);
	
		const FGameplayTagTableRow * Row = (const FGameplayTagTableRow*)DataTable->GetRowMap()["0"];
		if (Row)
		{
			check(Row->Tag == TEXT("Effect.Damage"));
		}
		return DataTable;
	}

	FGameplayTag GetTagForString(FStringView String)
	{
		return UGameplayTagsManager::Get().RequestGameplayTag(FName(String));
	}

	void GameplayTagTest_SimpleTest()
	{
		FName TagName = FName("Stack.DiminishingReturns");
		FGameplayTag Tag = UGameplayTagsManager::Get().RequestGameplayTag(TagName);
		TestTrueExpr(Tag.GetTagName() == TagName);
	}

	void GameplayTagTest_TagComparisonTest()
	{
		FGameplayTag EffectDamageTag = GetTagForString(TEXT("Effect.Damage"));
		FGameplayTag EffectDamage1Tag = GetTagForString(TEXT("Effect.Damage.Type1"));
		FGameplayTag EffectDamage2Tag = GetTagForString(TEXT("Effect.Damage.Type2"));
		FGameplayTag CueTag = GetTagForString(TEXT("GameplayCue.Burning"));
		FGameplayTag EmptyTag;

		TestTrueExpr(EffectDamage1Tag == EffectDamage1Tag);
		TestTrueExpr(EffectDamage1Tag != EffectDamage2Tag);
		TestTrueExpr(EffectDamage1Tag != EffectDamageTag);

		TestTrueExpr(EffectDamage1Tag.MatchesTag(EffectDamageTag));
		TestTrueExpr(!EffectDamage1Tag.MatchesTagExact(EffectDamageTag));
		TestTrueExpr(!EffectDamage1Tag.MatchesTag(EmptyTag));
		TestTrueExpr(!EffectDamage1Tag.MatchesTagExact(EmptyTag));
		TestTrueExpr(!EmptyTag.MatchesTag(EmptyTag));
		TestTrueExpr(!EmptyTag.MatchesTagExact(EmptyTag));

		TestTrueExpr(EffectDamage1Tag.RequestDirectParent() == EffectDamageTag);
	}

	void GameplayTagTest_TagContainerTest()
	{
		FGameplayTag EffectDamageTag = GetTagForString(TEXT("Effect.Damage"));
		FGameplayTag EffectDamage1Tag = GetTagForString(TEXT("Effect.Damage.Type1"));
		FGameplayTag EffectDamage2Tag = GetTagForString(TEXT("Effect.Damage.Type2"));
		FGameplayTag CueTag = GetTagForString(TEXT("GameplayCue.Burning"));
		FGameplayTag EmptyTag;

		FGameplayTagContainer EmptyContainer;

		FGameplayTagContainer TagContainer;
		TagContainer.AddTag(EffectDamage1Tag);
		TagContainer.AddTag(CueTag);

		FGameplayTagContainer ReverseTagContainer;
		ReverseTagContainer.AddTag(CueTag);
		ReverseTagContainer.AddTag(EffectDamage1Tag);
	
		FGameplayTagContainer TagContainer2;
		TagContainer2.AddTag(EffectDamage2Tag);
		TagContainer2.AddTag(CueTag);

		TestTrueExpr(TagContainer == TagContainer);
		TestTrueExpr(TagContainer == ReverseTagContainer);
		TestTrueExpr(TagContainer != TagContainer2);

		FGameplayTagContainer TagContainerCopy = TagContainer;

		TestTrueExpr(TagContainerCopy == TagContainer);
		TestTrueExpr(TagContainerCopy != TagContainer2);

		TagContainerCopy.Reset();
		TagContainerCopy.AppendTags(TagContainer);

		TestTrueExpr(TagContainerCopy == TagContainer);
		TestTrueExpr(TagContainerCopy != TagContainer2);

		TestTrueExpr(TagContainer.HasAny(TagContainer2));
		TestTrueExpr(TagContainer.HasAnyExact(TagContainer2));
		TestTrueExpr(!TagContainer.HasAll(TagContainer2));
		TestTrueExpr(!TagContainer.HasAllExact(TagContainer2));
		TestTrueExpr(TagContainer.HasAll(TagContainerCopy));
		TestTrueExpr(TagContainer.HasAllExact(TagContainerCopy));

		TestTrueExpr(TagContainer.HasAll(EmptyContainer));
		TestTrueExpr(TagContainer.HasAllExact(EmptyContainer));
		TestTrueExpr(!TagContainer.HasAny(EmptyContainer));
		TestTrueExpr(!TagContainer.HasAnyExact(EmptyContainer));

		TestTrueExpr(EmptyContainer.HasAll(EmptyContainer));
		TestTrueExpr(EmptyContainer.HasAllExact(EmptyContainer));
		TestTrueExpr(!EmptyContainer.HasAny(EmptyContainer));
		TestTrueExpr(!EmptyContainer.HasAnyExact(EmptyContainer));

		TestTrueExpr(!EmptyContainer.HasAll(TagContainer));
		TestTrueExpr(!EmptyContainer.HasAllExact(TagContainer));
		TestTrueExpr(!EmptyContainer.HasAny(TagContainer));
		TestTrueExpr(!EmptyContainer.HasAnyExact(TagContainer));

		TestTrueExpr(TagContainer.HasTag(EffectDamageTag));
		TestTrueExpr(!TagContainer.HasTagExact(EffectDamageTag));
		TestTrueExpr(!TagContainer.HasTag(EmptyTag));
		TestTrueExpr(!TagContainer.HasTagExact(EmptyTag));

		TestTrueExpr(EffectDamage1Tag.MatchesAny(FGameplayTagContainer(EffectDamageTag)));
		TestTrueExpr(!EffectDamage1Tag.MatchesAnyExact(FGameplayTagContainer(EffectDamageTag)));

		TestTrueExpr(EffectDamage1Tag.MatchesAny(TagContainer));

		FGameplayTagContainer FilteredTagContainer = TagContainer.FilterExact(TagContainer2);
		TestTrueExpr(FilteredTagContainer.HasTagExact(CueTag));
		TestTrueExpr(!FilteredTagContainer.HasTagExact(EffectDamage1Tag));

		FilteredTagContainer = TagContainer.Filter(FGameplayTagContainer(EffectDamageTag));
		TestTrueExpr(!FilteredTagContainer.HasTagExact(CueTag));
		TestTrueExpr(FilteredTagContainer.HasTagExact(EffectDamage1Tag));

		FilteredTagContainer.Reset();
		FilteredTagContainer.AppendMatchingTags(TagContainer, TagContainer2);
	
		TestTrueExpr(FilteredTagContainer.HasTagExact(CueTag));
		TestTrueExpr(!FilteredTagContainer.HasTagExact(EffectDamage1Tag));

		FGameplayTagContainer SingleTagContainer = EffectDamage1Tag.GetSingleTagContainer();
		FGameplayTagContainer ParentContainer = EffectDamage1Tag.GetGameplayTagParents();

		TestTrueExpr(SingleTagContainer.HasTagExact(EffectDamage1Tag));
		TestTrueExpr(SingleTagContainer.HasTag(EffectDamageTag));
		TestTrueExpr(!SingleTagContainer.HasTagExact(EffectDamageTag));

		TestTrueExpr(ParentContainer.HasTagExact(EffectDamage1Tag));
		TestTrueExpr(ParentContainer.HasTag(EffectDamageTag));
		TestTrueExpr(ParentContainer.HasTagExact(EffectDamageTag));

	}

	void GameplayTagTest_PerfTest()
	{
		FGameplayTag EffectDamageTag = GetTagForString(TEXT("Effect.Damage"));
		FGameplayTag EffectDamage1Tag = GetTagForString(TEXT("Effect.Damage.Type1"));
		FGameplayTag EffectDamage2Tag = GetTagForString(TEXT("Effect.Damage.Type2"));
		FGameplayTag CueTag = GetTagForString(TEXT("GameplayCue.Burning"));

		FGameplayTagContainer TagContainer;

		bool bResult = true;
		int32 SmallTest = 1000, LargeTest = 10000;

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d get tag"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				UGameplayTagsManager::Get().RequestGameplayTag(FName("Effect.Damage"));
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d container constructions"), SmallTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < SmallTest; i++)
			{
				TagContainer = FGameplayTagContainer();
				TagContainer.AddTag(EffectDamage1Tag);
				TagContainer.AddTag(EffectDamage2Tag);
				TagContainer.AddTag(CueTag);
				for (int32 j = 1; j <= 40; j++)
				{
					TagContainer.AddTag(GetTagForString(WriteToString<64>(TEXT("Expensive.Status.Tag.Type."), j)));
				}
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d container copy and move"), SmallTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < SmallTest; i++)
			{
				FGameplayTagContainer TagContainerNew(EffectDamageTag);
				TagContainerNew = TagContainer;

				FGameplayTagContainer MovedContainer = MoveTemp(TagContainerNew);

				bResult &= (MovedContainer.Num() == TagContainer.Num());
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d container addtag"), SmallTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < SmallTest; i++)
			{
				FGameplayTagContainer TagContainerNew;

				for (auto It = TagContainer.CreateConstIterator(); It; ++It)
				{
					TagContainerNew.AddTag(*It);
				}
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d container partial appends"), SmallTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < SmallTest; i++)
			{
				FGameplayTagContainer TagContainerNew(EffectDamage1Tag);

				TagContainerNew.AppendTags(TagContainer);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d container full appends"), SmallTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < SmallTest; i++)
			{
				FGameplayTagContainer TagContainerNew = TagContainer;

				TagContainerNew.AppendTags(TagContainer);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d container gets"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				FGameplayTagContainer TagContainerNew = EffectDamage1Tag.GetSingleTagContainer();
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d parent gets"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				FGameplayTagContainer TagContainerParents = EffectDamage1Tag.GetGameplayTagParents();
			}
		}
	
		FGameplayTagContainer TagContainer2;
		TagContainer2.AddTag(EffectDamage1Tag);
		TagContainer2.AddTag(EffectDamage2Tag);
		TagContainer2.AddTag(CueTag);

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d MatchesAnyExact checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= EffectDamage1Tag.MatchesAnyExact(TagContainer);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d MatchesAny checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= EffectDamage1Tag.MatchesAny(TagContainer);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d MatchesTag checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= EffectDamage1Tag.MatchesTag(EffectDamageTag);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d HasTagExact checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= TagContainer.HasTagExact(EffectDamage1Tag);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d HasTag checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= TagContainer.HasTag(EffectDamage1Tag);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d HasAll checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= TagContainer.HasAll(TagContainer2);
			}
		}

		{
			FScopeLogTime LogTimePtr(*FString::Printf(TEXT("%d HasAny checks"), LargeTest), nullptr, FScopeLogTime::ScopeLog_Milliseconds);
			for (int32 i = 0; i < LargeTest; i++)
			{
				bResult &= TagContainer.HasAny(TagContainer2);
			}
		}

		TestTrue(TEXT("Performance Tests succeeded"), bResult);
	}

};

IMPLEMENT_CUSTOM_SIMPLE_AUTOMATION_TEST(FGameplayTagTest, FGameplayTagTestBase, "System.GameplayTags.GameplayTag", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)
bool FGameplayTagTest::RunTest(const FString& Parameters)
{
	// Create Test Data 
	UDataTable* DataTable = CreateGameplayDataTable();

	UGameplayTagsManager::Get().PopulateTreeFromDataTable(DataTable);

	// Run Tests
	GameplayTagTest_SimpleTest();
	GameplayTagTest_TagComparisonTest();
	GameplayTagTest_TagContainerTest();
	GameplayTagTest_PerfTest();

	return !HasAnyErrors();
}

#endif //WITH_AUTOMATION_WORKER

TEST_CASE_NAMED(
	FSystem__GameplayTags__Manager__IsValidGameplayTagString,
	"System::GameplayTags::Manager::IsValidGameplayTagString",
	"[ApplicationContextMask][EngineFilter]")
{
	UGameplayTagsManager& Manager = UGameplayTagsManager::Get();

	SECTION("Valid tag: no modifications required")
	{
		FText Error;
		bool bValid = Manager.IsValidGameplayTagString(TEXT("Valid.Tag"), &Error);
		CHECK(bValid);
		CHECK(Error.IsEmpty());
	}

	SECTION("Tag with a leading period")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT(".Tag"), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}

	SECTION("Tag with a trailing period")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT("Tag."), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}

	SECTION("Tag with a leading space")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT(" Tag"), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}

	SECTION("Tag with a trailing space")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT("Tag "), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}

	SECTION("Tag with both leading and trailing unwanted characters")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT(" . Tag . "), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}

	SECTION("Tag with an invalid character inside")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT("Tag\t,Name"), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}

	SECTION("Tag with both leading and trailing unwanted characters and invalid characters inside")
	{
		FText Error;
		FString FixedTag;
		bool bValid = Manager.IsValidGameplayTagString(TEXT(" . Tag\t,Name . "), &Error, &FixedTag);
		CHECK(!bValid);
		CHECK_FALSE(Error.IsEmpty());
		INFO(Error.ToString());
		CHECK(Manager.IsValidGameplayTagString(FixedTag));
	}
}

#endif // WITH_TESTS