// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPath.h"
#include "GameplayTagContainer.h"

#include "GameplayTagRedirectors.generated.h"

#if WITH_EDITOR
class FBlake3;
#endif

/** A single redirect from a deleted tag to the new tag that should replace it */
USTRUCT()
struct FGameplayTagRedirect
{
	GENERATED_BODY()

public:
	FGameplayTagRedirect() { }

	UPROPERTY(EditAnywhere, Category = GameplayTags)
	FName OldTagName;

	UPROPERTY(EditAnywhere, Category = GameplayTags)
	FName NewTagName;

	friend inline bool operator==(const FGameplayTagRedirect& A, const FGameplayTagRedirect& B)
	{
		return A.OldTagName == B.OldTagName && A.NewTagName == B.NewTagName;
	}

	// This enables lookups by old tag name via FindByKey
	bool operator==(FName OtherOldTagName) const
	{
		return OldTagName == OtherOldTagName;
	}
};

/** This class is not thread safe and is protected against issues using the locks in GameplayTagManager */
class FGameplayTagRedirectors
{
public:
	static FGameplayTagRedirectors& Get();

	/** Sees if the tag name should be redirected to a different tag, returns false if there is no active redirect */
	bool RedirectTag(const FName& InTagName, FGameplayTag& OutTag) const;

	/** Adds redirects from the specified source. Faster than refreshing the entire list */
	void AddRedirectsFromSource(const struct FGameplayTagSource* Source);

	/** Refreshes the redirect map after a config change */
	void RefreshTagRedirects();

#if WITH_EDITOR
	/** Update the hasher with a deterministic hash of the existing redirects. Used for e.g. IncrementalCook keys. */
	void Hash(FBlake3& Hasher);

	/** 
	 * Find all source tag names from a target tag name. Only used in the cook so far. 
	 * The reverse lookup is done with a loop, there is nothing to make it fast so used only if you have to. Or better, 
	 * make a reverse lookup structure! 
	 */
	bool FindRedirectorsFromTargetTagName(const FName& TargetTagName, TArray<FName>& SourceTagNames) const;
#endif

private:
	FGameplayTagRedirectors();

	/** Adds a list of tag redirects to the map */
	void AddRedirects(const TArray<FGameplayTagRedirect>& Redirects);

	/** The map of ini-configured tag redirectors */
	TMap<FName, FGameplayTag> TagRedirects;
};
