// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"


#include "Iris/ReplicationSystem/NameTokenStore.h"

struct FGameplayTag;
namespace UE::Net
{

// For now, this is just a specialization of NameTokenStore
class FGameplayTagTokenStore : public FNameTokenStore
{
	UE_NONCOPYABLE(FGameplayTagTokenStore);
public:
	GAMEPLAYTAGS_API explicit FGameplayTagTokenStore(FNetTokenStore& TokenStore);

	// Create a NetToken for the provided name
	GAMEPLAYTAGS_API FNetToken GetOrCreateToken(FGameplayTag Tag);

	// Resolve NetToken, to resolve remote tokens RemoteTokenStoreState must be valid
	GAMEPLAYTAGS_API FGameplayTag ResolveToken(FNetToken Token, const FNetTokenStoreState* RemoteTokenStoreState = nullptr) const;

	static FName GetTokenStoreName()
	{
		return GameplayTokenStoreName;
	}

private:
	inline static FName GameplayTokenStoreName = "GameplayTagTokenStore";
};

}