// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "EngineGlobals.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "HAL/Platform.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/NameTypes.h"

/**
 * The public interface to this module, generally you should access the manager directly instead
 */
class IGameplayTagsModule : public IModuleInterface
{

public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGameplayTagsModule& Get()
	{
		static const FName GameplayTagModuleName("GameplayTags");
		return FModuleManager::LoadModuleChecked< IGameplayTagsModule >(GameplayTagModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		static const FName GameplayTagModuleName("GameplayTags");
		return FModuleManager::Get().IsModuleLoaded(GameplayTagModuleName);
	}

	/** Delegate for when assets are added to the tree */
	static GAMEPLAYTAGS_API FSimpleMulticastDelegate OnGameplayTagTreeChanged;

	/** Delegate that gets called after the settings have changed in the editor */
	static GAMEPLAYTAGS_API FSimpleMulticastDelegate OnTagSettingsChanged;

};

