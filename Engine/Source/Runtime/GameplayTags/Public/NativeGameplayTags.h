// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "GameplayTagContainer.h"
#include "GameplayTagsManager.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"

enum class ENativeGameplayTagToken { PRIVATE_USE_MACRO_INSTEAD };

namespace UE::GameplayTags::Private
{
	// Used to prevent people from putting UE_DEFINE_GAMEPLAY_TAG_STATIC and UE_DEFINE_GAMEPLAY_TAG in their headers.
	constexpr bool HasFileExtension(const char* File)
	{
		const char* It = File;
		while (*It)
			++It;
		return It[-1] == 'p' && It[-2] == 'p' && It[-3] == 'c' && It[-4] == '.';
	}
}

/**
 * Declares a native gameplay tag that is defined in a cpp with UE_DEFINE_GAMEPLAY_TAG to allow other modules or code to use the created tag variable.
 */
#define UE_DECLARE_GAMEPLAY_TAG_EXTERN(TagName) extern FNativeGameplayTag TagName;

/**
 * Defines a native gameplay tag with a comment that is externally declared in a header to allow other modules or code to use the created tag variable.
 */
#define UE_DEFINE_GAMEPLAY_TAG_COMMENT(TagName, Tag, Comment) FNativeGameplayTag TagName(UE_PLUGIN_NAME, UE_MODULE_NAME, Tag, TEXT(Comment), ENativeGameplayTagToken::PRIVATE_USE_MACRO_INSTEAD); static_assert(UE::GameplayTags::Private::HasFileExtension(__FILE__), "UE_DEFINE_GAMEPLAY_TAG_COMMENT can only be used in .cpp files, if you're trying to share tags across modules, use UE_DECLARE_GAMEPLAY_TAG_EXTERN in the public header, and UE_DEFINE_GAMEPLAY_TAG_COMMENT in the private .cpp");

/**
 * Defines a native gameplay tag with no comment that is externally declared in a header to allow other modules or code to use the created tag variable.
 */
#define UE_DEFINE_GAMEPLAY_TAG(TagName, Tag) FNativeGameplayTag TagName(UE_PLUGIN_NAME, UE_MODULE_NAME, Tag, TEXT(""), ENativeGameplayTagToken::PRIVATE_USE_MACRO_INSTEAD); static_assert(UE::GameplayTags::Private::HasFileExtension(__FILE__), "UE_DEFINE_GAMEPLAY_TAG can only be used in .cpp files, if you're trying to share tags across modules, use UE_DECLARE_GAMEPLAY_TAG_EXTERN in the public header, and UE_DEFINE_GAMEPLAY_TAG in the private .cpp");

/**
 * Defines a native gameplay tag such that it's only available to the cpp file you define it in.
 */
#define UE_DEFINE_GAMEPLAY_TAG_STATIC(TagName, Tag) static FNativeGameplayTag TagName(UE_PLUGIN_NAME, UE_MODULE_NAME, Tag, TEXT(""), ENativeGameplayTagToken::PRIVATE_USE_MACRO_INSTEAD); static_assert(UE::GameplayTags::Private::HasFileExtension(__FILE__), "UE_DEFINE_GAMEPLAY_TAG_STATIC can only be used in .cpp files, if you're trying to share tags across modules, use UE_DECLARE_GAMEPLAY_TAG_EXTERN in the public header, and UE_DEFINE_GAMEPLAY_TAG in the private .cpp");

#ifndef UE_INCLUDE_NATIVE_GAMEPLAYTAG_METADATA
	#define UE_INCLUDE_NATIVE_GAMEPLAYTAG_METADATA WITH_EDITOR && !UE_BUILD_SHIPPING
#endif

/**
 * Holds a gameplay tag that was registered during static construction of the module, and will
 * be unregistered when the module unloads.  Each registration is based on the native tag pointer
 * so even if two modules register the same tag and one is unloaded, the tag will still be registered
 * by the other one.
 */
class FNativeGameplayTag : public FNoncopyable
{
public:
	static GAMEPLAYTAGS_API FName NAME_NativeGameplayTag;

public:
	GAMEPLAYTAGS_API FNativeGameplayTag(FName PluginName, FName ModuleName, FName TagName, const FString& TagDevComment, ENativeGameplayTagToken);
	GAMEPLAYTAGS_API ~FNativeGameplayTag();

	operator FGameplayTag() const { return InternalTag; }

	FGameplayTag GetTag() const { return InternalTag; }

	FGameplayTagTableRow GetGameplayTagTableRow() const
	{
#if !UE_BUILD_SHIPPING
		ValidateTagRegistration();
#endif

#if WITH_EDITORONLY_DATA
		return FGameplayTagTableRow(InternalTag.GetTagName(), DeveloperComment);
#else
		return FGameplayTagTableRow(InternalTag.GetTagName());
#endif
	}

#if UE_INCLUDE_NATIVE_GAMEPLAYTAG_METADATA
	FName GetPlugin() const { return PluginName; }
	FName GetModuleName() const { return ModuleName; }
	FName GetModulePackageName() const { return ModulePackageName; }
#else
	FName GetModuleName() const { return NAME_NativeGameplayTag; }
	FName GetPlugin() const { return NAME_None; }
	FName GetModulePackageName() const { return NAME_None; }
#endif

private:
	FGameplayTag InternalTag;

#if !UE_BUILD_SHIPPING
	FName PluginName;
	FName ModuleName;
	FName ModulePackageName;
	mutable bool bValidated = false;

	GAMEPLAYTAGS_API void ValidateTagRegistration() const;
#endif

#if WITH_EDITORONLY_DATA
	FString DeveloperComment;
#endif

	static GAMEPLAYTAGS_API TSet<const class FNativeGameplayTag*>& GetRegisteredNativeTags();

	friend class UGameplayTagsManager;
};
